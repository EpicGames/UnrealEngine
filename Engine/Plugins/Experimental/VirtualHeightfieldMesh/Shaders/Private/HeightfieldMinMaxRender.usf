// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"

Texture2D SrcTexture;
int2 SrcTextureSize;

RWTexture2D<float4> DstTexture;
int2 DstTextureCoord;

// Packing swizzles so that RGBA texture (required format for UAV) can be read after mapping on CPU as BGRA
float4 PackMinMax(in float2 UnPacked)
{
	uint2 UnPackedUint = floor(UnPacked * 65535.f);
	float4 Packed = float4(
		(float)(UnPackedUint.y >> 8) / 255.f,
		(float)(UnPackedUint.x & 0xff) / 255.f,
		(float)(UnPackedUint.x >> 8) / 255.f,
		(float)(UnPackedUint.y & 0xff) / 255.f);
	return Packed;
}

float2 UnPackMinMax(in float4 Packed)
{
	uint4 PackedScaled = (uint4)floor(Packed *= 255.f);
	uint2 UnPackedScaled = uint2(PackedScaled.z << 8 | PackedScaled.y, PackedScaled.x << 8 | PackedScaled.w);
	float2 UnPacked = (float2)UnPackedScaled / 65535.f;
	return UnPacked;
}

// Min downsample excludes zero values to avoid empty pixels affecting the calculation
float Min4(in float4 Values, in float ReplaceZero)
{
	Values = select(Values == 0, ReplaceZero, Values);
	return min(min(min(Values.x, Values.y), Values.z), Values.w);
}

float Max4(in float4 Values)
{
	return max(max(max(Values.x, Values.y), Values.z), Values.w);
}

[numthreads(8, 8, 1)]
void MinMaxHeightCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	if (any(DispatchThreadId.xy * 2 >= (uint2)SrcTextureSize.xy))
		return;

#if INPUT_FORMAT_R16
	float T00 = SrcTexture[2 * DispatchThreadId.xy].x;
	float T01 = SrcTexture[2 * DispatchThreadId.xy + int2(0, 1)].x;
	float T10 = SrcTexture[2 * DispatchThreadId.xy + int2(1, 0)].x;
	float T11 = SrcTexture[2 * DispatchThreadId.xy + int2(1, 1)].x;
	float ValueMax = Max4(float4(T00, T01, T10, T11));
	float ValueMin = Min4(float4(T00, T01, T10, T11), ValueMax);
#elif INPUT_FORMAT_RG16
 	float2 T00 = SrcTexture[2 * DispatchThreadId.xy].xy;
 	float2 T01 = SrcTexture[2 * DispatchThreadId.xy + int2(0, 1)].xy;
 	float2 T10 = SrcTexture[2 * DispatchThreadId.xy + int2(1, 0)].xy;
 	float2 T11 = SrcTexture[2 * DispatchThreadId.xy + int2(1, 1)].xy;
	float ValueMax = Max4(float4(T00.y, T01.y, T10.y, T11.y));
	float ValueMin = Min4(float4(T00.x, T01.x, T10.x, T11.x), ValueMax);
#elif INPUT_FORMAT_RGBA8
	float2 T00 = UnPackMinMax(SrcTexture[2 * DispatchThreadId.xy]);
	float2 T01 = UnPackMinMax(SrcTexture[2 * DispatchThreadId.xy + int2(0, 1)]);
	float2 T10 = UnPackMinMax(SrcTexture[2 * DispatchThreadId.xy + int2(1, 0)]);
	float2 T11 = UnPackMinMax(SrcTexture[2 * DispatchThreadId.xy + int2(1, 1)]);
	float ValueMax = Max4(float4(T00.y, T01.y, T10.y, T11.y));
	float ValueMin = Min4(float4(T00.x, T01.x, T10.x, T11.x), ValueMax);
#endif

#if OUTPUT_FORMAT_RG16
	float4 Output = float4(ValueMin, ValueMax, 0, 0);
#elif OUTPUT_FORMAT_RGBA8
	float4 Output = PackMinMax(float2(ValueMin, ValueMax));
#endif

#if OUTPUT_TYPE_TEXEL
	DstTexture[DstTextureCoord] = Output;
#elif OUTPUT_TYPE_TEXTURE
	DstTexture[DispatchThreadId.xy] = Output;
#endif
}
