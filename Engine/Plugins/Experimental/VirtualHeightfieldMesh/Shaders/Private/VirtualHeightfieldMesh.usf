// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"	
#include "/Engine/Private/MortonCode.ush"	
//#include "/Engine/Private/ShaderPrint.ush"
#include "VirtualHeightfieldMesh.ush"	

groupshared uint NumGroupTasks;
RWStructuredBuffer<WorkerQueueInfo> RWQueueInfo;
RWCoherentBuffer(uint) RWQueueBuffer;
uint QueueBufferSizeMask;

RWBuffer<uint2> RWQuadBuffer;
Buffer<uint2> QuadBuffer;

RWBuffer<uint> RWIndirectArgsBuffer;
Buffer<uint> IndirectArgsBufferSRV;

RWStructuredBuffer<QuadRenderInstance> RWInstanceBuffer;
StructuredBuffer<QuadRenderInstance> InstanceBuffer;

RWBuffer<uint> RWFeedbackBuffer;

Texture2D<uint> PageTableTexture;

Texture2D<float4> HeightMinMaxTexture;
Texture2D<float4> LodBiasMinMaxTexture;
SamplerState MinMaxTextureSampler;
int MinMaxLevelOffset;

Texture2D<float> OcclusionTexture;
int OcclusionLevelOffset;

uint MaxLevel;
uint NumForceLoadLods;
uint PageTableFeedbackId;
uint NumPhysicalAddressBits;
float4 PageTableSize;
float4 PhysicalPageTransform;

float4 LodDistances;
float LodBiasScale;
float3 ViewOrigin;
float4 FrustumPlanes[5];
float4x4 UVToWorld;
float3 UVToWorldScale;

int NumIndices;


/** Unpack the virtual level for a PhysicalAddress entry in the virtual texture page table. */
uint GetVirtualLevelFromPhysicalAddress(uint InPhysicalAddress)
{
	// See packing in PageTableUpdate.usf
	return InPhysicalAddress & 0xf;
}

/** Compute physical UV from virtual UV in the tile with the given PhysicalAddress. */
float2 VirtualToPhysicalUV(float2 InVirtualUV, uint InPhysicalAddress, float4 InTransformFactors, uint InNumAddressBits)
{
	// See packing in PageTableUpdate.usf
	float PageX = (float)((InPhysicalAddress >> 4) & ((1u << InNumAddressBits) - 1));
	float PageY = (float)(InPhysicalAddress >> (4 + InNumAddressBits));
	float UVScale = 1.f / (float)(1u << GetVirtualLevelFromPhysicalAddress(InPhysicalAddress));

	float2 BaseUV = float2(PageX, PageY) * InTransformFactors.x;
	float2 PageUV = InVirtualUV * InTransformFactors.y;
	float2 BorderUV = InTransformFactors.z;
	float2 HalfTexelUV = InTransformFactors.w;

	return BaseUV + PageUV + BorderUV - HalfTexelUV;
}

/** Returns transform from virtual to physical UV in the tile with the given PhysicalAddress. Returns float3 where .xy is bias and .z is scale. */
float3 GetVirtualToPhysicalUVTransform(uint2 InPos, uint InLevel, uint InPhysicalAddress, float4 InTransformFactors, uint InNumAddressBits)
{
	uint LodShift = (uint)max((int)GetVirtualLevelFromPhysicalAddress(InPhysicalAddress) - (int)InLevel, 0);
	float PosDivider = 1.f / (float)(1u << LodShift);
	float2 MinVirtualUV = frac((float2)InPos * PosDivider);
	float2 MaxVirtualUV = MinVirtualUV + PosDivider;

	float2 MinPhysicalUV = VirtualToPhysicalUV(MinVirtualUV, InPhysicalAddress, InTransformFactors, InNumAddressBits);
	float2 MaxPhysicalUV = VirtualToPhysicalUV(MaxVirtualUV, InPhysicalAddress, InTransformFactors, InNumAddressBits);

	return float3(MinPhysicalUV, MaxPhysicalUV.x - MinPhysicalUV.x); // Assume Max.y - Min.y == Max.x - Min.x
}

/** Return false if location is marked as occluded in the occlusion texture. */
bool OcclusionTest(uint2 InPos, int InLevel)
{
	int OcclusionLevel = InLevel - OcclusionLevelOffset;
	if (OcclusionLevel < 0)
	{
		return true;
	}
	return OcclusionTexture.Load(uint3(InPos, OcclusionLevel)) == 0;
}

/** Unpack the values from the MinMaxHeight texture from the packed 8888 format. */
float2 UnPackMinMaxHeight(float4 InPacked)
{
	uint4 PackedScaled = (uint4)floor(InPacked *= 255.f);
	uint2 UnPackedScaled = uint2(PackedScaled.x << 8 | PackedScaled.y, PackedScaled.z << 8 | PackedScaled.w);
	float2 UnPacked = (float2)UnPackedScaled / 65535.f;
	return UnPacked;
}

/** Unpack the values from the MinMaxLodBias texture from the packed 8888 format. */
float2 UnPackMinMaxLodBias(float4 InPacked, float InLodBiasScale)
{
	return float2(CalculateBiasLod(InPacked.x, InLodBiasScale), CalculateBiasLod(InPacked.y, InLodBiasScale));
}

/** Return false if the AABB is completely outside one of the planes. */
bool PlaneTestAABB(float4 InPlanes[5], float3 InCenter, float3 InExtent)
{
	bool bPlaneTest = true;
	
	[unroll]
	for (uint PlaneIndex = 0; PlaneIndex < 5; ++PlaneIndex)
	{
		float3 PlaneSigns;
		PlaneSigns.x = InPlanes[PlaneIndex].x >= 0.f ? 1.f : -1.f;
		PlaneSigns.y = InPlanes[PlaneIndex].y >= 0.f ? 1.f : -1.f;
		PlaneSigns.z = InPlanes[PlaneIndex].z >= 0.f ? 1.f : -1.f;

		bool bInsidePlane = dot(InPlanes[PlaneIndex], float4(InCenter + InExtent * PlaneSigns, 1.0f)) > 0.f;
		bPlaneTest = bPlaneTest && bInsidePlane;
	}

	return bPlaneTest;
}

/* Return squared distance of closest distance between a point and a bounding box. */
float SquaredMinDistanceToAABB(float3 InPos, float3 InMin, float3 InMax, float3 InScale)
{
	float3 D1 = max(InMin - InPos, 0) * InScale;
	float3 D2 = max(InPos - InMax, 0) * InScale;
	return dot(D1, D1) + dot(D2, D2);
}

/* Return squared distance of furthest distance between a point and a bounding box. */
float SquaredMaxDistanceToAABB(float3 InPos, float3 InMin, float3 InMax, float3 InScale)
{
	float3 D = max(abs(InPos - InMin), (InPos - InMax)) * InScale;
	return dot(D, D);
}

/** Draw a bounding box using the ShaderDrawDebug system. */
void DebugDrawUVBox(float3 InUVMin, float3 InUVMax, float4x4 InTransform, float4 InColor)
{
#if 0 // Enable only if ShaderDrawDebug is enabled
	float3 WorldPos[8];
	WorldPos[0] = mul(float4(InUVMin.x, InUVMin.y, InUVMin.z, 1), InTransform);
	WorldPos[1] = mul(float4(InUVMax.x, InUVMin.y, InUVMin.z, 1), InTransform);
	WorldPos[2] = mul(float4(InUVMin.x, InUVMax.y, InUVMin.z, 1), InTransform);
	WorldPos[3] = mul(float4(InUVMax.x, InUVMax.y, InUVMin.z, 1), InTransform);
	WorldPos[4] = mul(float4(InUVMin.x, InUVMin.y, InUVMax.z, 1), InTransform);
	WorldPos[5] = mul(float4(InUVMax.x, InUVMin.y, InUVMax.z, 1), InTransform);
	WorldPos[6] = mul(float4(InUVMin.x, InUVMax.y, InUVMax.z, 1), InTransform);
	WorldPos[7] = mul(float4(InUVMax.x, InUVMax.y, InUVMax.z, 1), InTransform);

	AddQuadWS(WorldPos[0], WorldPos[2], WorldPos[3], WorldPos[1], InColor);
	AddQuadWS(WorldPos[4], WorldPos[6], WorldPos[7], WorldPos[5], InColor);
	AddLineWS(WorldPos[0], WorldPos[4], InColor, InColor);
	AddLineWS(WorldPos[1], WorldPos[5], InColor, InColor);
	AddLineWS(WorldPos[2], WorldPos[6], InColor, InColor);
	AddLineWS(WorldPos[3], WorldPos[7], InColor, InColor);
#endif
}

/**
 * Compute shader to initialize all buffers, including adding the lowest mip page(s) to the QuadBuffer.
 */

[numthreads(1, 1, 1)]
void InitBuffersCS()
{
	// Seed with one item in the queue.
	RWQueueInfo[0].Read = 0;
	RWQueueInfo[0].Write = 1;
	RWQueueInfo[0].NumActive = 1;

	RWQueueBuffer[0] = Pack(InitQuadItem(0, MaxLevel));

	// CullInstances indirect args
	RWIndirectArgsBuffer[0] = 0; // Increment this wave counter during CollectQuadsCS.
	RWIndirectArgsBuffer[1] = 1;
	RWIndirectArgsBuffer[2] = 1;
	RWIndirectArgsBuffer[3] = 0; // Increment this instance counter counter during CollectQuadsCS.

	// Clear virtual texture feedback counter or prime feedback buffer with first n levels.
	uint FeedbackCount = 0;

	for (uint Level = 0; Level < NumForceLoadLods; ++Level)
	{
		for (uint Y = 0; Y < (1u << Level); ++Y)
		{
			for (uint X = 0; X < (1u << Level); ++X)
			{
				uint LevelPlusOne = 1 + MaxLevel - Level;
				uint Feedback = X | (Y << 12) | (LevelPlusOne << 24) | PageTableFeedbackId;
				RWFeedbackBuffer[++FeedbackCount] = Feedback;
			}
		}
	}

	RWFeedbackBuffer[0] = FeedbackCount;
}

/**
 * Compute shader to traverse the virtual texture page table and generate an array of items to potentially render for a view.
 */

#if COMPILER_SUPPORTS_WAVE_SIZE
	WAVESIZE(32)
#endif
[numthreads(64, 1, 1)]
void CollectQuadsCS(
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint GroupIndex : SV_GroupIndex )
{
	// Persistant threads stay alive until the work queue is drained.
	bool bExit = false;
	while (!bExit)
	{
		// Sync and init group task count.
		NumGroupTasks = 0;
		GroupMemoryBarrierWithGroupSync();
		
		// Try and pull a task.
		int NumActive;
		InterlockedAdd(RWQueueInfo[0].NumActive, -1, NumActive);
		
		if (NumActive <= 0)
		{
			// No task pulled. Rewind.
			InterlockedAdd(RWQueueInfo[0].NumActive, 1, NumActive);
		}
		else
		{
			// Increment group task count for this loop.
			uint Dummy;
			InterlockedAdd(NumGroupTasks, 1, Dummy);

			// Read item to process from queue.
			uint Read;
			InterlockedAdd(RWQueueInfo[0].Read, 1, Read);
			
			uint PackedItem = RWQueueBuffer[Read & QueueBufferSizeMask];
			QuadItem Item = UnpackQuadItem(PackedItem);
			uint Address = Item.Address;
			uint2 Pos = MortonDecode(Address);
			uint Level = Item.Level;

			// Check if occluded.
			bool bOcclude = !OcclusionTest(Pos, Level);

			// Get UV bounding box
			float2 Scale = (float)(1u << Level) * PageTableSize.zw;
			
			float2 UV0 = ((float2)Pos + float2(0, 0)) * Scale;
			float2 UV1 = ((float2)Pos + float2(1, 1)) * Scale;
			
			float MinMaxTextureLevel = max((float)Level + (float)MinMaxLevelOffset, 0);
			float2 MinMaxHeight = UnPackMinMaxHeight(HeightMinMaxTexture.SampleLevel(MinMaxTextureSampler, UV0, MinMaxTextureLevel));
			float2 MinMaxLodBias = UnPackMinMaxLodBias(LodBiasMinMaxTexture.SampleLevel(MinMaxTextureSampler, UV0, MinMaxTextureLevel), LodBiasScale);
			
			float3 UVMin = float3(UV0, MinMaxHeight.x);
			float3 UVMax = float3(UV1, MinMaxHeight.y);

			float3 UVCenter = (UVMax + UVMin) * 0.5f;
			float3 UVExtent = UVMax - UVCenter;

			float MinDistanceSq = SquaredMinDistanceToAABB(ViewOrigin, UVMin, UVMax, UVToWorldScale);
			float MaxDistanceSq = SquaredMaxDistanceToAABB(ViewOrigin, UVMin, UVMax, UVToWorldScale);
			float MinDistanceLod = CalculateDistanceLod(MinDistanceSq, LodDistances);
			float MaxDistanceLod = CalculateDistanceLod(MaxDistanceSq, LodDistances);

			// Check if frustum culled
			bool bCull = !PlaneTestAABB(FrustumPlanes, UVCenter, UVExtent);

			bool bSubdivide = false;
			if (bCull || bOcclude)
			{
				// Store, but don't subdivide.
				DebugDrawUVBox(UVMin, UVMax, UVToWorld, float4(0, 0, 1, 1));
			}
			else if (Level > 0)
			{
				// Subdivide if minimum continuous lod can be less than the current level.
				bSubdivide = MinDistanceLod - MinMaxLodBias.y < (float)Level;
			}

			if (bSubdivide)
			{
				// Add children to queue.
				uint Write;
				InterlockedAdd(RWQueueInfo[0].Write, 4, Write);

				RWQueueBuffer[(Write + 0) & QueueBufferSizeMask] = Pack(InitQuadItem(Address * 4 + 0, Level - 1));
				RWQueueBuffer[(Write + 1) & QueueBufferSizeMask] = Pack(InitQuadItem(Address * 4 + 1, Level - 1));
				RWQueueBuffer[(Write + 2) & QueueBufferSizeMask] = Pack(InitQuadItem(Address * 4 + 2, Level - 1));
				RWQueueBuffer[(Write + 3) & QueueBufferSizeMask] = Pack(InitQuadItem(Address * 4 + 3, Level - 1));

				InterlockedAdd(RWQueueInfo[0].NumActive, 4, NumActive);
			}
			else
			{
				// Add to output list.
				uint PhysicalAddress = PageTableTexture.Load(int3(Pos, Level));

				uint Write;
				InterlockedAdd(RWIndirectArgsBuffer[3], 1, Write);
				InterlockedMax(RWIndirectArgsBuffer[0], ((Write + 1) + 63) / 64);

				RWQuadBuffer[Write] = Pack(InitQuadRenderItem(Pos, Level, PhysicalAddress, bCull || bOcclude));

				// Add all possible pages that vertex shader might read to the virtual texture feedback buffer.
				int MinFeedbackLevel = (int)floor(clamp(MinDistanceLod - MinMaxLodBias.y, Level, MaxLevel));
				int MaxFeedbackLevel = (int)ceil(clamp(MaxDistanceLod - MinMaxLodBias.x, Level, MaxLevel));
				uint NumFeedbackItems = (uint)max(MaxFeedbackLevel - MinFeedbackLevel + 1, 0);

				uint FeedbackPos;
				InterlockedAdd(RWFeedbackBuffer[0], NumFeedbackItems, FeedbackPos);
				
				for (int FeedbackLevel = MinFeedbackLevel; FeedbackLevel <= MaxFeedbackLevel; ++FeedbackLevel)
				{
					// Note that our general virtual texture feedback buffer convention is to write Level+1
					uint LevelPlusOne = FeedbackLevel + 1;
					uint LodShift = FeedbackLevel - Level;
					RWFeedbackBuffer[FeedbackPos + FeedbackLevel] = (Pos.x >> LodShift) | ((Pos.y >> LodShift) << 12) | (LevelPlusOne << 24) | PageTableFeedbackId;
				}

				// Debug draw the bounds.
				if (!(bCull || bOcclude))
				{
					DebugDrawUVBox(UVMin, UVMax, UVToWorld, float4(1, 0, 0, 1));
				}
			}
		}

		// Exit if no work was found.
		DeviceMemoryBarrier();
		if (NumGroupTasks == 0)
		{
			bExit = true;
		}
	}
}

/**
 * Initialise the indirect args for the final culled indirect draw call.
 */

[numthreads(1, 1, 1)]
void InitInstanceBufferCS()
{
	RWIndirectArgsBuffer[0] = NumIndices;
	RWIndirectArgsBuffer[1] = 0; // Increment this counter during CullInstancesCS.
	RWIndirectArgsBuffer[2] = 0;
	RWIndirectArgsBuffer[3] = 0;
	RWIndirectArgsBuffer[4] = 0;
}

/**
 * Cull the potentially visible render items for a view and generate the final buffer of instances to render.
 */

[numthreads(64, 1, 1)]
void CullInstancesCS( uint3 DispatchThreadId : SV_DispatchThreadID )
{
	uint QuadIndex = DispatchThreadId.x;
	if (QuadIndex >= IndirectArgsBufferSRV[3])
		return;

	uint2 PackedItem = QuadBuffer[QuadIndex];
	QuadRenderItem Item = UnpackQuadRenderItem(PackedItem);
	uint2 Pos = Item.Pos;
	uint Level = Item.Level;

#if REUSE_CULL
	// Reuse main view culling flag set in the Collect pass.
	bool bCull = Item.bCull;
#else
	// Cull against planes for this view.
	float2 Scale = (float)(1u << Level) * PageTableSize.zw;
	float2 UV0 = ((float2)Pos + float2(0, 0)) * Scale;
	float2 UV1 = ((float2)Pos + float2(1, 1)) * Scale;

	float MinMaxTextureLevel = max((float)Level + (float)MinMaxLevelOffset, 0);
	float2 MinMaxHeight = UnPackMinMaxHeight(HeightMinMaxTexture.SampleLevel(MinMaxTextureSampler, UV0, MinMaxTextureLevel));

	float3 UVMin = float3(UV0, MinMaxHeight.x);
	float3 UVMax = float3(UV1, MinMaxHeight.y);

	float3 UVCenter = (UVMax + UVMin) * 0.5f;
	float3 UVExtent = UVMax - UVCenter;

	bool bCull = !PlaneTestAABB(FrustumPlanes, UVCenter, UVExtent);
#endif

	if (!bCull)
	{
		// Add to final render intance list.
		QuadRenderInstance OutInstance;
		OutInstance.PosLevelPacked = Pos.x | (Pos.y << 12) | (Level << 24);

		// Unpack physical address for first sample into full LocalToPhysicalUV here ready for use by vertex shader. 
		OutInstance.UVTransform.xyz = GetVirtualToPhysicalUVTransform(Pos, Level, Item.PhysicalAddress, PhysicalPageTransform, NumPhysicalAddressBits);

		uint Write;
		InterlockedAdd(RWIndirectArgsBuffer[1], 1, Write);
		RWInstanceBuffer[Write] = OutInstance;
	}
}
