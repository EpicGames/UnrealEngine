// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Structures used by VirtualHeightfieldMesh.
 * These need to be kept in sync with any C++ buffer definitions in VirtualHeightfieldMeshSceneProxy.cpp
 */

/** Structure used for tracking work queues in persistent wave style shaders. */
struct WorkerQueueInfo
{
	uint Read;
	uint Write;
	int	NumActive;
};

/** Item description used when traversing the virtual page table quad tree. Packs as uint so store in Buffer declared as uint. */
struct QuadItem
{
	uint Address;
	uint Level;
};

uint Pack(QuadItem Item)
{
	uint PackedItem = 0;
	PackedItem |= Item.Address;
	PackedItem |= Item.Level << 24;
	return PackedItem;
}

QuadItem InitQuadItem(uint Address, uint Level)
{
	QuadItem Item;
	Item.Address = Address;
	Item.Level = Level;
	return Item;
}

QuadItem UnpackQuadItem(uint PackedItem)
{
	QuadItem Item;
	Item.Address = PackedItem & 0x00FFFFFF;
	Item.Level = PackedItem >> 24;
	return Item;
}

/** Description for items that are output by the quad tree collect stage. Packs as uint2 so store in Buffer declared as uint2. */
struct QuadRenderItem
{
	uint2 Pos;
	uint Level;
	uint PhysicalAddress;
	bool bCull;
};

uint2 Pack(QuadRenderItem Item)
{
	uint2 PackedItem = 0;
	PackedItem.x |= Item.Pos.x;
	PackedItem.x |= Item.Pos.y << 12;
	PackedItem.x |= Item.Level << 24;
	PackedItem.y |= Item.PhysicalAddress & 0x000FFFFF;
	PackedItem.y |= Item.bCull ? 1 << 20 : 0;
	return PackedItem;
}

QuadRenderItem InitQuadRenderItem(uint2 Pos, uint Level, uint PhysicalAddress, bool bCull)
{
	QuadRenderItem Item;
	Item.Pos = Pos;
	Item.Level = Level;
	Item.PhysicalAddress = PhysicalAddress;
	Item.bCull = bCull;
	return Item;
}

QuadRenderItem UnpackQuadRenderItem(uint2 PackedItem)
{
	QuadRenderItem Item;
	Item.Pos.x = PackedItem.x & 0x00000FFF;
	Item.Pos.y = (PackedItem.x & 0x00FFF000) >> 12;
	Item.Level = PackedItem.x >> 24;
	Item.PhysicalAddress = PackedItem.y & 0x000FFFFF;
	Item.bCull = ((PackedItem.y & 0x00100000) >> 20) == 1;
	return Item;
}

/** Final render instance description used by the DrawInstancedIndirect(). */
struct QuadRenderInstance
{
	float3 UVTransform;
	uint PosLevelPacked;
};

uint2 UnpackPos(QuadRenderInstance Item)
{
	return uint2(Item.PosLevelPacked & 0x00000FFF, (Item.PosLevelPacked & 0x00FFF000) >> 12);
}

uint UnpackLevel(QuadRenderInstance Item)
{
	return Item.PosLevelPacked >> 24;
}


/** Shared helper function to calculate a LOD based on distance from camera. */
float CalculateDistanceLod(float InDistanceSq, float4 InLodFactors)
{
	float ScaledDistance = sqrt(InDistanceSq) * InLodFactors.w;
	float LodForDistance0 = saturate(ScaledDistance / (InLodFactors.x * InLodFactors.y));
	float LodForDistanceN = log2(1 + max((ScaledDistance / InLodFactors.x - InLodFactors.y), 0)) / log2(InLodFactors.z);
	return LodForDistance0 + LodForDistanceN;
}

/** Shared helper function to calculate a LOD bias based on our LodBias texture value. */
float CalculateBiasLod(float InValue, float InLodBiasScale)
{
	return max((InValue - 0.05f) * InLodBiasScale - 1.f, 0);
}
