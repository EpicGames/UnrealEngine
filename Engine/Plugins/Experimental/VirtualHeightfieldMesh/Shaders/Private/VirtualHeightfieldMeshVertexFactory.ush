// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/VertexFactoryCommon.ush"
#include "/Engine/Private/VirtualTextureCommon.ush"
#include "VirtualHeightfieldMesh.ush"	

#define GRID_SIZE (VHM.NumQuadsPerTileSide+1)

StructuredBuffer<QuadRenderInstance> InstanceBuffer;
float3 LodViewOrigin;
float4 LodDistances;

#undef GetInstanceIdFromVF
#define GetInstanceIdFromVF(VFInput)				(VFInput.InstanceId)

/** Per-vertex inputs. No vertex buffers are bound. */
struct FVertexFactoryInput
{
	uint InstanceId : SV_InstanceID;
	uint VertexId : SV_VertexID;
	VF_MOBILE_MULTI_VIEW_DECLARE_INPUT_BLOCK()
};

/** Cached intermediates that would otherwise have to be computed multiple times. Avoids relying on the compiler to optimize out redundant operations. */
struct FVertexFactoryIntermediates
{
	float2 LocalUV;
	float3 VTPos;
	float3 LocalPos;
	float3 WorldNormal;
	/** Cached primitive and instance data */
	FSceneDataIntermediates SceneData; 
};

FPrimitiveSceneData GetPrimitiveData(FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.SceneData.Primitive;
}

/** Attributes to interpolate from the vertex shader to the pixel shader. */
struct FVertexFactoryInterpolantsVSToPS
{
#if NUM_TEX_COORD_INTERPOLATORS
	float4 TexCoords[(NUM_TEX_COORD_INTERPOLATORS + 1) / 2] : TEXCOORD0;
#endif
};

/** Fetch the lod bias for the location. */
float GetLodBias(float2 InUV, float InLodBiasScale)
{
	return CalculateBiasLod(VHM.LodBiasTexture.SampleLevel(VHM.LodBiasSampler, InUV, 0), InLodBiasScale);
}

/** Helper to morph the UV location of a vertex. First snaps down InMorphFactorFloor LOD levels and then morphs InMorphFactorFrac towards the next LOD level. */
float2 MorphVertex(float2 InLocalUV, uint InGridSize, uint InMorphFactorFloor, float InMorphFactorFrac)
{
	float2 MorphedUV = InLocalUV;
	
	// Full morph levels
	float MorphGridSize = InGridSize >> InMorphFactorFloor;
	float2 MorphGridDimensions = float2(MorphGridSize, 1.f / MorphGridSize);
	float2 MorphOffset1 = frac(InLocalUV * MorphGridDimensions.x) * MorphGridDimensions.y;
	MorphedUV -= MorphOffset1;
	
	// Partial morph to next level
	float2 MorphOffset2 = frac(MorphedUV * MorphGridDimensions.x * 0.5f) * MorphGridDimensions.y * 2.f;
	MorphedUV -= MorphOffset2 * InMorphFactorFrac;
	
	return MorphedUV;
}

/** Compute the intermediates for a given vertex. */
FVertexFactoryIntermediates GetVertexFactoryIntermediates(FVertexFactoryInput Input)
{
	FVertexFactoryIntermediates Intermediates;
	Intermediates.SceneData = VF_GPUSCENE_GET_INTERMEDIATES(Input);

	const QuadRenderInstance Item = InstanceBuffer[Input.InstanceId];
	const uint2 Pos = UnpackPos(Item);
	const uint Level = UnpackLevel(Item);
	
	const float3 LocalUVTransform = Item.UVTransform;

	uint2 VertexCoord = uint2(Input.VertexId % GRID_SIZE, Input.VertexId / GRID_SIZE);
	float2 LocalUV = (float2)VertexCoord / (float)(GRID_SIZE - 1);

	// Calculate vertex UV details before morphing
	float2 XY = ((float2)Pos + LocalUV) * (float)(1u << Level);
	float2 NormalizedPos = (XY * VHM.PageTableSize.zw);
	float SampleLevel = Level;

	// Sample height once to approximate distance and morph the LocalUV.
	{
		float2 LocalPhysicalUV = LocalUVTransform.xy + LocalUV * LocalUVTransform.z;
		float Height = VHM.HeightTexture.SampleLevel(VHM.HeightSampler, LocalPhysicalUV, 0);
		float3 WorldPos = mul(float4(NormalizedPos, Height, 1), VHM.VirtualHeightfieldToWorld).xyz;
		float DistanceSq = dot(LodViewOrigin - WorldPos, LodViewOrigin - WorldPos);
		float LodForDistance = CalculateDistanceLod(DistanceSq, LodDistances);
		float LodBias = GetLodBias(NormalizedPos, VHM.LodBiasScale);
		
		// Clamp between the LOD level for this instance and the max LOD. 
		// Note that the culling phase should already ensure that LOD >= Level.
		float LodClamped = clamp(LodForDistance - LodBias, (float)Level, VHM.MaxLod);
		
		// Number of levels that we need to morph.
		float LodMorphFloor = floor(LodClamped) - (float)Level;
		float LodMorphCeil = ceil(LodClamped) - (float)Level;
		float LodMorphFrac = LodClamped - (LodMorphFloor + (float)Level);

		// NOTE: Removing fractional continuous LOD here.
		// This is because fractional locations come away from the surface of the triangles that they interpolate and we see the resultant surface shimmer.
		// A fix for this while keeping fractional LOD is to use some sort of triangle barycentric interpolation when sampling the height texture instead of bilinear.
		// But snapping to units here looks OK.
		LodMorphFrac = 0;

		// Apply morph to vertex postion.
		LocalUV = MorphVertex(LocalUV, GRID_SIZE - 1, (uint)LodMorphFloor, LodMorphFrac);
		// Adjust other position variables to match the new morphed position.
		XY = ((float2)Pos + LocalUV) * (float)(1u << Level);
		NormalizedPos = (XY * VHM.PageTableSize.zw);

		// Use LOD level as sample level but bias to keep sample level in a good range.
		SampleLevel = max(0, LodClamped - 0.5f);
	}
	
	// Sample height from virtual texture.
	VTUniform Uniform = VTUniform_Unpack(VHM.VTPackedUniform);
	Uniform.vPageBorderSize -= .5f * VHM.PhysicalTextureSize.y; // Half texel offset is used in VT write and in sampling because we want texel locations to match landscape vertices.
	VTPageTableUniform PageTableUniform = VTPageTableUniform_Unpack(VHM.VTPackedPageTableUniform0, VHM.VTPackedPageTableUniform1);
	VTPageTableResult VTResult0 = TextureLoadVirtualPageTableLevel(VHM.PageTableTexture, PageTableUniform, NormalizedPos, VTADDRESSMODE_CLAMP, VTADDRESSMODE_CLAMP, floor(SampleLevel));
	float2 UV0 = VTComputePhysicalUVs(VTResult0, 0, Uniform);
	float Height0 = VHM.HeightTexture.SampleLevel(VHM.HeightSampler, UV0, 0);
	VTPageTableResult VTResult1 = TextureLoadVirtualPageTableLevel(VHM.PageTableTexture, PageTableUniform, NormalizedPos, VTADDRESSMODE_CLAMP, VTADDRESSMODE_CLAMP, ceil(SampleLevel));
	float2 UV1 = VTComputePhysicalUVs(VTResult1, 0, Uniform);
	float Height1 = VHM.HeightTexture.SampleLevel(VHM.HeightSampler, UV1, 0);
	float Height = lerp(Height0.x, Height1.x, frac(SampleLevel));

	// Position in space of virtual texture volume
	Intermediates.VTPos = float3(NormalizedPos, Height);

	// Position in local space
	Intermediates.LocalPos = mul(float4(Intermediates.VTPos, 1), VHM.VirtualHeightfieldToLocal).xyz;
	
	Intermediates.LocalUV = NormalizedPos;

	Intermediates.WorldNormal = float3(0, 0, 1);

	return Intermediates;
}

float3 VertexFactoryGetPreviousInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates);
float3 VertexFactoryGetInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates);

/** Converts from vertex factory specific input to a FMaterialVertexParameters, which is used by vertex shader material inputs. */
FMaterialVertexParameters GetMaterialVertexParameters(
	FVertexFactoryInput Input, 
	FVertexFactoryIntermediates Intermediates, 
	float3 WorldPosition, 
	half3x3 TangentToLocal,
	bool bIsPreviousFrame = false)
{
	FMaterialVertexParameters Result = MakeInitializedMaterialVertexParameters();

	Result.SceneData = Intermediates.SceneData;
	
	Result.WorldPosition = WorldPosition;
	if (bIsPreviousFrame)
	{
		Result.PositionInstanceSpace = VertexFactoryGetPreviousInstanceSpacePosition(Input, Intermediates);
	}
	else
	{
		Result.PositionInstanceSpace = VertexFactoryGetInstanceSpacePosition(Input, Intermediates);
	}
	Result.PositionPrimitiveSpace = Result.PositionInstanceSpace; // No support for instancing, so instance == primitive

	// needs fixing!
	Result.TangentToWorld = mul(TangentToLocal, (float3x3)VHM.VirtualHeightfieldToWorld);
	Result.TangentToWorld[2] = Intermediates.WorldNormal;

	Result.PreSkinnedPosition = WorldPosition;// Intermediates.WorldPosPreDisplacement.xyz;
	Result.PreSkinnedNormal = float3(0, 0, 1);

#if VF_VIRTUAL_HEIGHFIELD_MESH
	//Result.Displacement = Intermediates.Displacement;
#endif

#if NUM_MATERIAL_TEXCOORDS_VERTEX
	UNROLL
	for (int CoordinateIndex = 0; CoordinateIndex < NUM_MATERIAL_TEXCOORDS_VERTEX; CoordinateIndex++)
	{
		Result.TexCoords[CoordinateIndex] = Intermediates.LocalUV;
	}
#endif  //NUM_MATERIAL_TEXCOORDS_VERTEX

	Result.LWCData = MakeMaterialLWCData(Result);

	return Result;
}

/** Get ID in GPU Scene. We don't implement support because we create/consume our own instancing buffer. */
uint GetPrimitiveId(FVertexFactoryInterpolantsVSToPS Interpolants)
{
	return 0;
}

/** Get ID in the GPU Scene. */
uint VertexFactoryGetPrimitiveId(FVertexFactoryInterpolantsVSToPS Interpolants)
{
	return GetPrimitiveId(Interpolants);
}

/** Computes the world space position of this vertex. */
float4 VertexFactoryGetWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	FDFMatrix LocalToWorld = GetPrimitiveData(Intermediates).LocalToWorld;
	return TransformLocalToTranslatedWorld(Intermediates.LocalPos, LocalToWorld);
}

// local position relative to instance
float3 VertexFactoryGetInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.LocalPos; // No support for instancing, so instance == primitive
}

/** Computes the world space position of this vertex. */
float4 VertexFactoryGetRasterizedWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, float4 InWorldPosition)
{
#if 0 // todo: rough vertex cull outside bounds
	float4x4 TranslatedWorldToLocal = DFFastToTranslatedWorld(GetPrimitiveData(Intermediates).WorldToLocal, ResolvedView.PreViewTranslation);
	float3 LocalObjectBoundsMin = GetPrimitiveData(Intermediates).LocalObjectBoundsMin - 0.05;
	float3 LocalObjectBoundsMax = GetPrimitiveData(Intermediates).LocalObjectBoundsMax + 0.05;
	float3 LocalPos = mul(float4(InWorldPosition.xyz, 1), TranslatedWorldToLocal).xyz;
	float Divider = (any(LocalPos > LocalObjectBoundsMax) || any(LocalPos < LocalObjectBoundsMin)) ? 0 : 1;
	return float4(InWorldPosition.xyz, InWorldPosition.w / Divider);
#else
	return InWorldPosition;
#endif
}

/** Computes the world space position used by vertex lighting for this vertex. */
float3 VertexFactoryGetPositionForVertexLighting(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, float3 TranslatedWorldPosition)
{
	return TranslatedWorldPosition;
}

/** Computes the world space position of this vertex last frame. */
float4 VertexFactoryGetPreviousWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return DFTransformLocalToTranslatedWorld(Intermediates.LocalPos, GetPrimitiveData(Intermediates).PreviousLocalToWorld, ResolvedView.PrevPreViewTranslation);
}

/** Computes the instance space position of this vertex last frame. */
float3 VertexFactoryGetPreviousInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.LocalPos; // No support for instancing, so instance == primitive
}

/** Computes the world space normal of this vertex. */
float3 VertexFactoryGetWorldNormal(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.WorldNormal;
}

/** Get the 3x3 tangent basis vectors for this vertex factory. This vertex factory will calculate the binormal on-the-fly. */
half3x3 VertexFactoryGetTangentToLocal(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return half3x3(1, 0, 0, 0, 1, 0, 0, 0, 1);
}

/** Get the translated bounding sphere for this primitive. */
float4 VertexFactoryGetTranslatedPrimitiveVolumeBounds(FVertexFactoryInterpolantsVSToPS Interpolants)
{
	FPrimitiveSceneData PrimitiveData = GetPrimitiveData(GetPrimitiveId(Interpolants));
	return float4(DFFastToTranslatedWorld(PrimitiveData.ObjectWorldPosition, ResolvedView.PreViewTranslation), PrimitiveData.ObjectRadius);
}

#if NUM_TEX_COORD_INTERPOLATORS

void SetUV(inout FVertexFactoryInterpolantsVSToPS Interpolants, uint UVIndex, float2 InValue)
{
	FLATTEN
	if (UVIndex % 2)
	{
		Interpolants.TexCoords[UVIndex / 2].zw = InValue;
	}
	else
	{
		Interpolants.TexCoords[UVIndex / 2].xy = InValue;
	}
}

float2 GetUV(FVertexFactoryInterpolantsVSToPS Interpolants, uint UVIndex)
{
	float4 UVVector = Interpolants.TexCoords[UVIndex / 2];
	return UVIndex % 2 ? UVVector.zw : UVVector.xy;
}

#endif

/** Constructs values that need to be interpolated from the vertex shader to the pixel shader. */
FVertexFactoryInterpolantsVSToPS VertexFactoryGetInterpolantsVSToPS(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, FMaterialVertexParameters VertexParameters)
{
	FVertexFactoryInterpolantsVSToPS Interpolants;

	// Initialize the whole struct to 0
	// Really only the last two components of the packed UVs have the opportunity to be uninitialized
	Interpolants = (FVertexFactoryInterpolantsVSToPS)0;

#if NUM_TEX_COORD_INTERPOLATORS
	float2 CustomizedUVs[NUM_TEX_COORD_INTERPOLATORS];
	GetMaterialCustomizedUVs(VertexParameters, CustomizedUVs);
	GetCustomInterpolators(VertexParameters, CustomizedUVs);

	UNROLL
	for (int CoordinateIndex = 0; CoordinateIndex < NUM_TEX_COORD_INTERPOLATORS; CoordinateIndex++)
	{
		SetUV(Interpolants, CoordinateIndex, CustomizedUVs[CoordinateIndex]);
	}
#endif

	return Interpolants;
}

/** Converts from vertex factory specific interpolants to a FMaterialPixelParameters, which is used by material inputs. */
FMaterialPixelParameters GetMaterialPixelParameters(FVertexFactoryInterpolantsVSToPS Interpolants, float4 SvPosition)
{
	// GetMaterialPixelParameters is responsible for fully initializing the result
	FMaterialPixelParameters Result = MakeInitializedMaterialPixelParameters();

#if NUM_TEX_COORD_INTERPOLATORS
	UNROLL
	for (uint CoordinateIndex = 0; CoordinateIndex < NUM_TEX_COORD_INTERPOLATORS; CoordinateIndex++)
	{
		Result.TexCoords[CoordinateIndex] = GetUV(Interpolants, CoordinateIndex);
	}
#endif	//NUM_MATERIAL_TEXCOORDS

	Result.TwoSidedSign = 0;
	Result.PrimitiveId = GetPrimitiveId(Interpolants);

	return Result;
}

#include "/Engine/Private/VertexFactoryDefaultInterface.ush"
