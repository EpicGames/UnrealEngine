// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeightfieldMinMaxRender.h"

#include "GlobalShader.h"
#include "RenderGraphUtils.h"
#include "DataDrivenShaderPlatformInfo.h"

/** Shader for MinMax downsample passes. */
class FMinMaxTextureCS : public FGlobalShader
{
public:
	SHADER_USE_PARAMETER_STRUCT(FMinMaxTextureCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, SrcTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, DstTexture)
		SHADER_PARAMETER(FIntPoint, SrcTextureSize)
		SHADER_PARAMETER(FIntPoint, DstTextureCoord)
	END_SHADER_PARAMETER_STRUCT()
};

/** Enum of packing formats for reading/writing height. */
enum EHeightFormat
{
	EHeight_R16,
	EHeight_RG16,
	EHeight_RGBA8,
};

/** Enum of pass output types. */
enum EOutputType
{
	/* Output entire downsampled texture. */
	EOutputTexture, 
	/* Output a single final downsampled texel to a specified location in the destination texture. */
	EOutputTexel,
};

/** Templatized shader specializations. */
template< EHeightFormat InputFormat, EHeightFormat OutputFormat, EOutputType OutputType >
class TMinMaxTextureCS : public FMinMaxTextureCS
{
public:
	DECLARE_SHADER_TYPE(TMinMaxTextureCS, Global);

	TMinMaxTextureCS() 
	{}

	TMinMaxTextureCS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMinMaxTextureCS(Initializer)
	{}

	static bool ShouldCompilePermutation(FGlobalShaderPermutationParameters const& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static inline void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		switch(InputFormat)
		{
		case EHeight_R16: OutEnvironment.SetDefine(TEXT("INPUT_FORMAT_R16"), 1); break;
		case EHeight_RG16: OutEnvironment.SetDefine(TEXT("INPUT_FORMAT_RG16"), 1); break;
		case EHeight_RGBA8: OutEnvironment.SetDefine(TEXT("INPUT_FORMAT_RGBA8"), 1); break;
		}
		switch (OutputFormat)
		{
		case EHeight_R16: OutEnvironment.SetDefine(TEXT("OUTPUT_FORMAT_R16"), 1); break;
		case EHeight_RG16: OutEnvironment.SetDefine(TEXT("OUTPUT_FORMAT_RG16"), 1); break;
		case EHeight_RGBA8: OutEnvironment.SetDefine(TEXT("OUTPUT_FORMAT_RGBA8"), 1); break;
		}
		switch (OutputType)
		{
		case EOutputTexture: OutEnvironment.SetDefine(TEXT("OUTPUT_TYPE_TEXTURE"), 1); break;
		case EOutputTexel: OutEnvironment.SetDefine(TEXT("OUTPUT_TYPE_TEXEL"), 1); break;
		}
	}
};

/** Implementations of the used shader variations. */
#define IMPLEMENT_MINMAX_SHADER_TYPE(Input, Output, Type, ShaderName) \
	typedef TMinMaxTextureCS<Input, Output, Type> TMinMaxTextureCS##ShaderName; \
	IMPLEMENT_SHADER_TYPE(template<>, TMinMaxTextureCS##ShaderName, TEXT("/Plugin/VirtualHeightfieldMesh/Private/HeightfieldMinMaxRender.usf"), TEXT("MinMaxHeightCS"), SF_Compute);

IMPLEMENT_MINMAX_SHADER_TYPE(EHeight_R16, EHeight_RG16, EOutputTexture, _R16ToRG16);
IMPLEMENT_MINMAX_SHADER_TYPE(EHeight_RG16, EHeight_RG16, EOutputTexture, _RG16ToRG16);
IMPLEMENT_MINMAX_SHADER_TYPE(EHeight_RG16, EHeight_RGBA8, EOutputTexel, _RG16ToRGBA8_Texel);
IMPLEMENT_MINMAX_SHADER_TYPE(EHeight_RGBA8, EHeight_RGBA8, EOutputTexture, _RGBA8ToRGBA8);

namespace
{
	/** Initial pass that reads from R16 height. */
	void AddFirstMinMaxPass(FRDGBuilder& GraphBuilder, FRDGTextureSRVRef Src, FIntPoint SrcSize, int32 SrcMipLevel, FRDGTextureUAVRef Dst)
	{
		TShaderMapRef<TMinMaxTextureCS_R16ToRG16> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));

		FMinMaxTextureCS::FParameters* Parameters = GraphBuilder.AllocParameters<FMinMaxTextureCS::FParameters>();
		Parameters->SrcTexture = Src;
		Parameters->DstTexture = Dst;
		Parameters->SrcTextureSize = SrcSize;

		const FIntVector GroupCount((SrcSize.X / 2 + 7) / 8, (SrcSize.Y / 2 + 7) / 8, 1);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("MinMaxPass"),
			ComputeShader, Parameters, GroupCount);
	}

	/** Generic pass that assumes that Src and Dst are consecutive mips in a single texture resource. */
	template<typename ShaderType>
	void AddMinMaxMipPass(FRDGBuilder& GraphBuilder, FRDGTextureSRVRef Src, FIntPoint SrcSize, int32 SrcMipLevel, FRDGTextureUAVRef Dst)
	{
		TShaderMapRef<ShaderType> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));

		FMinMaxTextureCS::FParameters* Parameters = GraphBuilder.AllocParameters<FMinMaxTextureCS::FParameters>();
		Parameters->SrcTexture = Src;
		Parameters->DstTexture = Dst;
		Parameters->SrcTextureSize = SrcSize;

		const FIntVector GroupCount((SrcSize.X / 2 + 7) / 8, (SrcSize.Y / 2 + 7) / 8, 1);

		ClearUnusedGraphResources(ComputeShader, Parameters);

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("MinMaxPass"),
			Parameters,
			ERDGPassFlags::Compute,
			[Parameters, ComputeShader, GroupCount](FRHICommandList& RHICmdList)
			{
				FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, *Parameters, GroupCount);
			});
	}

	/** Final pass that writes packed result to a specific location in a destination texture. */
	void AddLastMinMaxPass(FRDGBuilder& GraphBuilder, FRDGTextureSRVRef Src, FIntPoint SrcSize, int32 SrcMipLevel, FRDGTextureUAVRef Dst, FIntPoint DstCoord)
	{
		TShaderMapRef<TMinMaxTextureCS_RG16ToRGBA8_Texel> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));

		FMinMaxTextureCS::FParameters* Parameters = GraphBuilder.AllocParameters<FMinMaxTextureCS::FParameters>();
		Parameters->SrcTexture = Src;
		Parameters->DstTexture = Dst;
		Parameters->SrcTextureSize = SrcSize;
		Parameters->DstTextureCoord = DstCoord;

		const FIntVector GroupCount((SrcSize.X / 2 + 7) / 8, (SrcSize.Y / 2 + 7) / 8, 1);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("MinMaxPass"),
			ComputeShader, Parameters, GroupCount);
	}
}

namespace VirtualHeightfieldMesh
{
	void DownsampleMinMaxAndCopy(FRDGBuilder& GraphBuilder, FRDGTexture* SrcTexture, FIntPoint SrcSize, FRDGTextureUAV* DstTextureUAV, FIntPoint DstCoord)
	{
		const FIntPoint DownsampleTextureSize = FIntPoint(SrcSize.X / 2, SrcSize.X / 2);
		const int32 NumMips = FMath::FloorLog2(FMath::Max(DownsampleTextureSize.X, DownsampleTextureSize.Y)) + 1;
		check(NumMips > 1);

		const ETextureCreateFlags TextureFlags = TexCreate_ShaderResource | TexCreate_UAV | TexCreate_RenderTargetable;
		const FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(DownsampleTextureSize, PF_R16G16B16A16_UNORM, FClearValueBinding::None, TextureFlags, NumMips);
		FRDGTextureRef DownsampleTexture = GraphBuilder.CreateTexture(Desc, TEXT("DownsampleTexture"));

		FIntPoint Size = SrcSize;
		for (int32 MipLevel = 0; MipLevel < NumMips; ++MipLevel)
		{ 
			const bool bFirstPass = MipLevel == 0;
			const bool bLastPass = MipLevel == NumMips - 1;

			FRDGTextureSRVRef SRV;
			if (bFirstPass)
			{
				SRV = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(SrcTexture));
			}
			else
			{
				SRV = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(DownsampleTexture, MipLevel - 1));
			}

			FRDGTextureUAVRef UAV;
			if (bLastPass)
			{
				UAV = DstTextureUAV;
			}
			else
			{
				UAV = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(DownsampleTexture, MipLevel));
			}
			
			if (bFirstPass)
			{
				AddFirstMinMaxPass(GraphBuilder, SRV, Size, MipLevel, UAV);
			}
			else if (!bLastPass)
			{
				AddMinMaxMipPass<TMinMaxTextureCS_RG16ToRG16>(GraphBuilder, SRV, Size, MipLevel, UAV);
			}
			else
			{
				AddLastMinMaxPass(GraphBuilder, SRV, Size, MipLevel, UAV, DstCoord);
			}

			Size.X = FMath::Max(Size.X / 2, 1);
			Size.Y = FMath::Max(Size.Y / 2, 1);
		}
	}

	void GenerateMinMaxTextureMips(FRDGBuilder& GraphBuilder, FRDGTexture* Texture, FIntPoint SrcSize, int32 NumMips)
	{
		FIntPoint Size = SrcSize;
		for (int32 MipLevel = 1; MipLevel < NumMips; ++MipLevel)
		{
			FRDGTextureSRVRef SRV = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(Texture, MipLevel - 1));
			FRDGTextureUAVRef UAV = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(Texture, MipLevel));

			AddMinMaxMipPass<TMinMaxTextureCS_RGBA8ToRGBA8>(GraphBuilder, SRV, Size, MipLevel, UAV);

			Size.X = FMath::Max(Size.X / 2, 1);
			Size.Y = FMath::Max(Size.Y / 2, 1);
		}
	}
}
