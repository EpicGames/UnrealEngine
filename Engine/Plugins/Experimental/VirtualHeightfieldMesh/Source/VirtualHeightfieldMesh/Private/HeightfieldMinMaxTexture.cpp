// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeightfieldMinMaxTexture.h"

#include "Engine/Texture2D.h"
#include "HeightfieldMinMaxTextureNotify.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HeightfieldMinMaxTexture)

UHeightfieldMinMaxTexture::UHeightfieldMinMaxTexture(const FObjectInitializer& ObjectInitializer)
	: UObject(ObjectInitializer)
	, MaxCPULevels(5)
{
}

#if WITH_EDITOR

void UHeightfieldMinMaxTexture::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	static FName MaxCPULevelsName = GET_MEMBER_NAME_CHECKED(UHeightfieldMinMaxTexture, MaxCPULevels);
	if (PropertyChangedEvent.Property && (PropertyChangedEvent.Property->GetFName() == MaxCPULevelsName))
	{
		RebuildCPUTextureData();

		VirtualHeightfieldMesh::NotifyComponents(this);
	}
}

void UHeightfieldMinMaxTexture::BuildTexture(FHeightfieldMinMaxTextureBuildDesc const& InBuildDesc)
{
	RebuildMinMaxTexture(InBuildDesc);
	RebuildLodBiasTexture(InBuildDesc);
	RebuildCPUTextureData();

	// Notify all dependent components.
	VirtualHeightfieldMesh::NotifyComponents(this);
}

void UHeightfieldMinMaxTexture::RebuildMinMaxTexture(FHeightfieldMinMaxTextureBuildDesc const& InBuildDesc)
{
	Texture = NewObject<UTexture2D>(this, TEXT("HeightMinMaxTexture"));

	FTextureFormatSettings Settings;
	Settings.CompressionSettings = TC_EditorIcon;
	Settings.CompressionNone = true;
	Settings.SRGB = false;

	Texture->Filter = TF_Nearest;
	Texture->MipGenSettings = TMGS_LeaveExistingMips;
	Texture->MipLoadOptions = ETextureMipLoadOptions::AllMips;
	Texture->NeverStream = true;
	Texture->SetLayerFormatSettings(0, Settings);
	Texture->Source.Init(InBuildDesc.SizeX, InBuildDesc.SizeY, 1, InBuildDesc.NumMips, TSF_BGRA8, InBuildDesc.Data);

	Texture->PostEditChange();
}

void UHeightfieldMinMaxTexture::RebuildLodBiasTexture(FHeightfieldMinMaxTextureBuildDesc const& InBuildDesc)
{
	TArray<uint8> LodBiasTextureData;
	TArray<uint32> MinMaxLodBiasTextureData;

	// Build the LodBias texture data
	{
		// Allocate two work buffers (we will need second for blur passes).
		TArray<uint16> ScratchBuffers[2];
		ScratchBuffers[0].AddDefaulted(InBuildDesc.SizeX * InBuildDesc.SizeY);
		ScratchBuffers[1].AddDefaulted(InBuildDesc.SizeX * InBuildDesc.SizeY);
		int32 ScratchBufferWriteIndex = 0;
		
		// Extract MinMax values from source data and store the differences.
		{
			uint16 MaxHeightDiff = 1;
			for (uint32 Index = 0; Index < InBuildDesc.SizeX * InBuildDesc.SizeY; ++Index)
			{
				const uint16 Min = InBuildDesc.Data[Index * 4 + 2] * 256 + InBuildDesc.Data[Index * 4 + 3];
				const uint16 Max = InBuildDesc.Data[Index * 4 + 0] * 256 + InBuildDesc.Data[Index * 4 + 1];
				const uint16 HeightDiff = FMath::Max(Max, Min) - Min;
				ScratchBuffers[ScratchBufferWriteIndex][Index] = HeightDiff;
				MaxHeightDiff = FMath::Max(MaxHeightDiff, HeightDiff);
			}

			// Normalize the data against the maximum value.
			for (uint32 Index = 0; Index < InBuildDesc.SizeX * InBuildDesc.SizeY; ++Index)
			{
				const uint16 NormalizedRange = (uint16)(65535.f * ((float)ScratchBuffers[0][Index] / (float)MaxHeightDiff));
				ScratchBuffers[ScratchBufferWriteIndex][Index] = NormalizedRange;
			}
		}

		// Ping-pong blur to reduce maximum difference between any two texels.
		// This is to remove sharp gradients in the data which will generate lod transitions that are too aggressive for the lod algorithm to stitch correctly.
		{
			const uint32 NumBlurPasses = 20;
			const uint16 MaxGradient = 65535 / 10;

			for (uint32 BlurIndex = 0; BlurIndex < NumBlurPasses; ++BlurIndex)
			{
				const int32 ScratchBufferReadIndex = ScratchBufferWriteIndex;
				ScratchBufferWriteIndex = 1 - ScratchBufferWriteIndex;
				
				for (int32 Y = 0; Y < (int32)InBuildDesc.SizeY; ++Y)
				{
					for (int32 X = 0; X < (int32)InBuildDesc.SizeX; ++X)
					{
						// Get maximum value in the 3x3 neighborhood.
						uint16 MaxValue = 0;
						for (int32 YOffset = -1; YOffset <= 1; ++YOffset)
						{
							int32 YCoord = FMath::Clamp<int32>(Y + YOffset, 0, InBuildDesc.SizeY - 1);
							for (int32 XOffset = -1; XOffset <= 1; ++XOffset)
							{
								int32 XCoord = FMath::Clamp<int32>(X + XOffset, 0, InBuildDesc.SizeX - 1);

								MaxValue = FMath::Max(MaxValue, ScratchBuffers[ScratchBufferReadIndex][InBuildDesc.SizeY * YCoord + XCoord]);
							}
						}

						// Clamp to within MaxGradient of MaxValue.
						const uint16 MinValue = MaxValue > MaxGradient ? MaxValue - MaxGradient : 0;
						const uint16 CurrentValue = ScratchBuffers[ScratchBufferReadIndex][InBuildDesc.SizeY * Y + X];
						const uint16 NewValue = FMath::Max(CurrentValue, MinValue);
						ScratchBuffers[ScratchBufferWriteIndex][InBuildDesc.SizeY * Y + X] = NewValue;
					}
				}
			}
		}

		// Copy final ping-pong buffer to result data.
		{
			const int32 ScratchBufferReadIndex = ScratchBufferWriteIndex;

			LodBiasTextureData.AddDefaulted(InBuildDesc.SizeX * InBuildDesc.SizeY);
			for (uint32 Index = 0; Index < InBuildDesc.SizeX * InBuildDesc.SizeY; ++Index)
			{
				LodBiasTextureData[Index] = ScratchBuffers[ScratchBufferReadIndex][Index] >> 8;
			}
		}
	}

	// Create new texture
	{
		LodBiasTexture = NewObject<UTexture2D>(this, TEXT("LodBiasTexture"));

		FTextureFormatSettings Settings;
		Settings.CompressionSettings = TC_EditorIcon;
		Settings.CompressionNone = true;
		Settings.SRGB = false;

		LodBiasTexture->Filter = TF_Nearest;
		LodBiasTexture->MipGenSettings = TMGS_NoMipmaps;
		LodBiasTexture->SetLayerFormatSettings(0, Settings);
		LodBiasTexture->Source.Init(InBuildDesc.SizeX, InBuildDesc.SizeY, 1, 1, TSF_G8, LodBiasTextureData.GetData());

		LodBiasTexture->PostEditChange();
	}

	// Build top mip of MinMaxLodBias texture. 
	// Include a texel border. This will account for any bleed in the vertex factory code where we bilinear sample the LodBiasTexture.
	{
		// Reserve the expected entries assuming square mips. This may be an overestimate.
		MinMaxLodBiasTextureData.Reserve(((1 << (2 * InBuildDesc.NumMips)) - 1) / 3);

		for (int32 Y = 0; Y < (int32)InBuildDesc.SizeY; ++Y)
		{
			for (int32 X = 0; X < (int32)InBuildDesc.SizeX; ++X)
			{
				uint32 MaxValue = 0;
				uint32 MinValue = 255;
				for (int32 YOffset = -1; YOffset <= 1; ++YOffset)
				{
					const int32 YCoord = FMath::Clamp<int32>(Y + YOffset, 0, InBuildDesc.SizeY - 1);
					for (int32 XOffset = -1; XOffset <= 1; ++XOffset)
					{
						const int32 XCoord = FMath::Clamp<int32>(X + XOffset, 0, InBuildDesc.SizeX - 1);
						const uint32 LodBiasValue = LodBiasTextureData[InBuildDesc.SizeY * YCoord + XCoord];

						MaxValue = FMath::Max(MaxValue, LodBiasValue);
						MinValue = FMath::Min(MinValue, LodBiasValue);
					}
				}

				// We pack min and max values to map to the red and green channels in our BGRA8 texture.
				// It would be better to use an RG8 here but that isn't supported by texture import.
				MinMaxLodBiasTextureData.Add((MaxValue << 8) | (MinValue << 16) | 0xFF000000);
			}
		}
	}

	// Build mips with MinMax filter
	{
		int32 SizeX = InBuildDesc.SizeX;
		int32 SizeY = InBuildDesc.SizeY;
		uint32 const* SourceData = MinMaxLodBiasTextureData.GetData();
		for (uint32 MipIndex = 1; MipIndex < InBuildDesc.NumMips; ++MipIndex)
		{
			const int32 SourceSizeX = SizeX;
			const int32 SourceSizeY = SizeY;
			
			SizeX = FMath::Max(SizeX >> 1, 1);
			SizeY = FMath::Max(SizeY >> 1, 1);

			for (int32 Y = 0; Y < SizeY; ++Y)
			{
				for (int32 X = 0; X < SizeX; ++X)
				{
					const int32 X0 = FMath::Min(X * 2, SourceSizeX - 1);
					const int32 X1 = FMath::Min(X * 2 + 1, SourceSizeX - 1);
					const int32 Y0 = FMath::Min(Y * 2, SourceSizeY - 1);
					const int32 Y1 = FMath::Min(Y * 2 + 1, SourceSizeY - 1);
					const uint32 S00 = *(SourceData + Y0 * SourceSizeY + X0);
					const uint32 S10 = *(SourceData + Y0 * SourceSizeY + X1);
					const uint32 S01 = *(SourceData + Y1 * SourceSizeY + X0);
					const uint32 S11 = *(SourceData + Y1 * SourceSizeY + X1);

					// MinMax filter assumes BGRA8 layout.
					const uint32 MaxValue = FMath::Max(FMath::Max(FMath::Max((S00 & 0x00FF00), (S10 & 0x00FF00)), (S01 & 0x00FF00)), (S11 & 0x00FF00));
					const uint32 MinValue = FMath::Min(FMath::Min(FMath::Min((S00 & 0xFF0000), (S10 & 0xFF0000)), (S01 & 0xFF0000)), (S11 & 0xFF0000));
					MinMaxLodBiasTextureData.Add(MaxValue | MinValue | 0xFF000000);
				}
			}

			SourceData += SourceSizeX * SourceSizeY;
		}
	}

	// Create new texture
	{
		LodBiasMinMaxTexture = NewObject<UTexture2D>(this, TEXT("LodBiasMinMaxTexture"));

		FTextureFormatSettings Settings;
		Settings.CompressionSettings = TC_EditorIcon;
		Settings.CompressionNone = true;
		Settings.SRGB = false;

		LodBiasMinMaxTexture->Filter = TF_Nearest;
		LodBiasMinMaxTexture->MipGenSettings = TMGS_LeaveExistingMips;
		LodBiasMinMaxTexture->MipLoadOptions = ETextureMipLoadOptions::AllMips;
		LodBiasMinMaxTexture->NeverStream = true;
		LodBiasMinMaxTexture->SetLayerFormatSettings(0, Settings);
		LodBiasMinMaxTexture->Source.Init(InBuildDesc.SizeX, InBuildDesc.SizeY, 1, InBuildDesc.NumMips, TSF_BGRA8, (uint8*)MinMaxLodBiasTextureData.GetData());

		LodBiasMinMaxTexture->PostEditChange();
	}
}

void UHeightfieldMinMaxTexture::RebuildCPUTextureData()
{
	TextureData.Reset();
	TextureDataMips.Reset();

	if (Texture != nullptr && Texture->Source.IsValid() && MaxCPULevels > 0)
	{
		const int32 NumTextureMips = Texture->Source.GetNumMips();
		const int32 NumCPUMips = FMath::Min(NumTextureMips, MaxCPULevels);
		const int32 BaseMipIndex = NumTextureMips - NumCPUMips;

		const int32 TextureSizeX = Texture->Source.GetSizeX();
		const int32 TextureSizeY = Texture->Source.GetSizeY();
		TextureDataSize.X = FMath::Max(TextureSizeX >> BaseMipIndex, 1);
		TextureDataSize.Y = FMath::Max(TextureSizeY >> BaseMipIndex, 1);

		// Reserve the expected entries assuming square mips. This may be an overestimate.
		TextureData.Reserve(((1 << (2 * NumCPUMips)) - 1) / 3);
		TextureDataMips.Reserve(NumCPUMips);

		// Iterate the Texture mips and extract min/max values to store in a flat array.
		for (int32 MipIndex = BaseMipIndex; MipIndex < NumTextureMips; ++MipIndex)
		{
			TextureDataMips.Add(TextureData.Num());

			TArray64<uint8> MipData;
			if (Texture->Source.GetMipData(MipData, MipIndex))
			{
				for (int32 Index = 0; Index < MipData.Num(); Index += 4)
				{
					float Min = (float)(MipData[Index + 2] * 256 + MipData[Index + 3]) / 65535.f;
					float Max = (float)(MipData[Index + 0] * 256 + MipData[Index + 1]) / 65535.f;
					TextureData.Add(FVector2D(Min, Max));
				}
			}
		}

		TextureData.Shrink();
	}
}

#endif

