// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeightfieldMinMaxTextureMaterialExpression.h"

#include "EditorSupportDelegates.h"
#include "Engine/Texture2D.h"
#include "HeightfieldMinMaxTexture.h"
#include "MaterialCompiler.h"
#include "Materials/MaterialExpressionUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HeightfieldMinMaxTextureMaterialExpression)

#define LOCTEXT_NAMESPACE "VirtualHeightfieldMeshModule"

UMaterialExpressionHeightfieldMinMaxTexture::UMaterialExpressionHeightfieldMinMaxTexture(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText NAME_VirtualHeightfieldMesh;
		FConstructorStatics()
			: NAME_VirtualHeightfieldMesh(LOCTEXT("VirtualHeightfieldMesh", "VirtualHeightfieldMesh"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_VirtualHeightfieldMesh);
#endif
}

#if WITH_EDITOR

void UMaterialExpressionHeightfieldMinMaxTexture::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetName() == TEXT("MinMaxTexture"))
	{
		if (MinMaxTexture != nullptr && MinMaxTexture->Texture != nullptr)
		{
			SamplerType = MaterialExpressionUtils::GetSamplerTypeForTexture(MinMaxTexture->Texture);

			FEditorSupportDelegates::ForcePropertyWindowRebuild.Broadcast(this);
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

int32 UMaterialExpressionHeightfieldMinMaxTexture::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (MinMaxTexture != nullptr && MinMaxTexture->Texture != nullptr)
	{
		return Compiler->Texture(MinMaxTexture->Texture, SamplerType);
	}

	return CompilerError(Compiler, TEXT("Requires valid texture"));
}

int32 UMaterialExpressionHeightfieldMinMaxTexture::CompilePreview(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (MinMaxTexture != nullptr && MinMaxTexture->Texture != nullptr)
	{
		int32 TextureReferenceIndex = INDEX_NONE;
		int32 TextureCodeIndex = Compiler->Texture(MinMaxTexture->Texture, TextureReferenceIndex, SamplerType, SSM_Wrap_WorldGroupSettings, TMVM_None);

		return Compiler->TextureSample(TextureCodeIndex, Compiler->TextureCoordinate(0, false, false), SamplerType);
	}

	return CompilerError(Compiler, TEXT("Requires valid texture"));
}

void UMaterialExpressionHeightfieldMinMaxTexture::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("HeightfieldMinMaxTextureObject"));
}

EMaterialValueType UMaterialExpressionHeightfieldMinMaxTexture::GetOutputValueType(int32 OutputIndex)
{
	return MCT_Texture2D;
}

#endif // WITH_EDITOR

UObject* UMaterialExpressionHeightfieldMinMaxTexture::GetReferencedTexture() const
{
	return MinMaxTexture != nullptr ? MinMaxTexture->Texture : nullptr;
}

#undef LOCTEXT_NAMESPACE

