// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Materials/MaterialExpression.h"
#include "HeightfieldMinMaxTextureMaterialExpression.generated.h"

struct FPropertyChangedEvent;
class UHeightfieldMinMaxTexture;
enum EMaterialSamplerType : int;

/** Node which outputs a texture object contained in a UHeightfieldMinMaxTexture. */
UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionHeightfieldMinMaxTexture : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MaterialExpressionTexture)
	TObjectPtr<UHeightfieldMinMaxTexture> MinMaxTexture;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MaterialExpressionTexture)
	TEnumAsByte<EMaterialSamplerType> SamplerType;

protected:
#if WITH_EDITOR
	//~ Begin UObject Interface
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface
#endif

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual int32 CompilePreview(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual bool CanReferenceTexture() const override { return true; }
#endif
	virtual UObject* GetReferencedTexture() const override;
	//~ End UMaterialExpression Interface
};
