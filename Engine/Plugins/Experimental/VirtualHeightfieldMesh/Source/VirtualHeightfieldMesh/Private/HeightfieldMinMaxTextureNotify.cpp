// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeightfieldMinMaxTextureNotify.h"

#include "UObject/UObjectIterator.h"
#include "VirtualHeightfieldMeshComponent.h"

namespace VirtualHeightfieldMesh
{
#if WITH_EDITOR

	void NotifyComponents(UHeightfieldMinMaxTexture const* MinMaxTexture)
	{
		for (TObjectIterator<UVirtualHeightfieldMeshComponent> It; It; ++It)
		{
			if (It->GetMinMaxTexture() == MinMaxTexture)
			{
				It->MarkRenderStateDirty();
			}
		}
	}

#endif
}
