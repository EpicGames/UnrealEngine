// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


class UHeightfieldMinMaxTexture;

namespace VirtualHeightfieldMesh
{
#if WITH_EDITOR
	/**
	 * Find any UVirtualHeightfieldMeshComponent that reference this UHeightfieldMinMaxTexture and mark them dirty.
	 * We need to do this after editing the UHeightfieldMinMaxTexture settings.
	 */
	void NotifyComponents(UHeightfieldMinMaxTexture const* MinMaxTexture);
#endif
}
