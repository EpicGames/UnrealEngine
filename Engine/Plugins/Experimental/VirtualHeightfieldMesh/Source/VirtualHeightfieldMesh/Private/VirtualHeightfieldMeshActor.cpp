// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualHeightfieldMeshActor.h"

#include "Components/BoxComponent.h"
#include "VirtualHeightfieldMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VirtualHeightfieldMeshActor)

AVirtualHeightfieldMesh::AVirtualHeightfieldMesh(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	RootComponent = VirtualHeightfieldMeshComponent = CreateDefaultSubobject<UVirtualHeightfieldMeshComponent>(TEXT("VirtualHeightfieldMeshComponent"));

#if WITH_EDITORONLY_DATA
	// Add box for visualization of bounds
	Box = CreateDefaultSubobject<UBoxComponent>(TEXT("Bounds"));
	Box->SetBoxExtent(FVector(.5f, .5f, .5f), false);
	Box->SetRelativeTransform(FTransform(FVector(.5f, .5f, .5f)));
	Box->bDrawOnlyIfSelected = true;
	Box->SetIsVisualizationComponent(true);
	Box->SetCollisionEnabled(ECollisionEnabled::NoCollision);
	Box->SetCanEverAffectNavigation(false);
	Box->CanCharacterStepUpOn = ECanBeCharacterBase::ECB_No;
	Box->SetGenerateOverlapEvents(false);
	Box->SetupAttachment(VirtualHeightfieldMeshComponent);
#endif
}

