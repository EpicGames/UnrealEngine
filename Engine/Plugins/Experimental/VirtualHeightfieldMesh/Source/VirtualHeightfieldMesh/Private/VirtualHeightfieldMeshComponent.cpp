// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualHeightfieldMeshComponent.h"

#include "Components/RuntimeVirtualTextureComponent.h"
#include "HeightfieldMinMaxTexture.h"
#include "SceneInterface.h"
#include "VirtualHeightfieldMeshEnable.h"
#include "VirtualHeightfieldMeshSceneProxy.h"
#include "VT/RuntimeVirtualTexture.h"
#include "VT/RuntimeVirtualTextureVolume.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VirtualHeightfieldMeshComponent)

UVirtualHeightfieldMeshComponent::UVirtualHeightfieldMeshComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	CastShadow = true;
	bCastContactShadow = false;
	bUseAsOccluder = true;
	bAffectDynamicIndirectLighting = false;
	bAffectDistanceFieldLighting = false;
	bNeverDistanceCull = true;
	bEnableAutoLODGeneration = false;
	Mobility = EComponentMobility::Static;
}

void UVirtualHeightfieldMeshComponent::OnRegister()
{
	VirtualTextureRef = VirtualTexture.Get();

	URuntimeVirtualTextureComponent* RuntimeVirtualTextureComponent = VirtualTextureRef != nullptr ? ToRawPtr(VirtualTextureRef->VirtualTextureComponent) : nullptr;
	if (RuntimeVirtualTextureComponent)
	{
		// Bind to delegate so that we dirty render state whenever RuntimeVirtualTextureComponent is moved.
		RuntimeVirtualTextureComponent->TransformUpdated.AddUObject(this, &UVirtualHeightfieldMeshComponent::OnVirtualTextureTransformUpdate);
		// Bind to delegate so that RuntimeVirtualTextureComponent will pull hide flags from this object.
		RuntimeVirtualTextureComponent->GetHidePrimitivesDelegate().AddUObject(this, &UVirtualHeightfieldMeshComponent::GatherHideFlags);
		RuntimeVirtualTextureComponent->MarkRenderStateDirty();
	}

	Super::OnRegister();
}

void UVirtualHeightfieldMeshComponent::OnUnregister()
{
	URuntimeVirtualTextureComponent* RuntimeVirtualTextureComponent = VirtualTextureRef != nullptr ? ToRawPtr(VirtualTextureRef->VirtualTextureComponent) : nullptr;
	if (RuntimeVirtualTextureComponent)
	{
		RuntimeVirtualTextureComponent->TransformUpdated.RemoveAll(this);
		RuntimeVirtualTextureComponent->GetHidePrimitivesDelegate().RemoveAll(this);
		RuntimeVirtualTextureComponent->MarkRenderStateDirty();
	}

	VirtualTextureRef = nullptr;

	Super::OnUnregister();
}

void UVirtualHeightfieldMeshComponent::ApplyWorldOffset(const FVector& InOffset, bool bWorldShift)
{
	Super::ApplyWorldOffset(InOffset, bWorldShift);
	MarkRenderStateDirty();
}

ARuntimeVirtualTextureVolume* UVirtualHeightfieldMeshComponent::GetVirtualTextureVolume() const
{
	return VirtualTextureRef;
}

URuntimeVirtualTexture* UVirtualHeightfieldMeshComponent::GetVirtualTexture() const
{
	URuntimeVirtualTextureComponent* RuntimeVirtualTextureComponent = VirtualTextureRef != nullptr ? ToRawPtr(VirtualTextureRef->VirtualTextureComponent) : nullptr;
	return RuntimeVirtualTextureComponent ? RuntimeVirtualTextureComponent->GetVirtualTexture() : nullptr;
}

FTransform UVirtualHeightfieldMeshComponent::GetVirtualTextureTransform() const
{
	URuntimeVirtualTextureComponent* RuntimeVirtualTextureComponent = VirtualTextureRef != nullptr ? ToRawPtr(VirtualTextureRef->VirtualTextureComponent) : nullptr;
	return RuntimeVirtualTextureComponent ? RuntimeVirtualTextureComponent->GetComponentTransform() * RuntimeVirtualTextureComponent->GetTexelSnapTransform() : FTransform::Identity;
}

bool UVirtualHeightfieldMeshComponent::IsVisible() const
{
	return
		Super::IsVisible() &&
		GetVirtualTexture() != nullptr &&
		GetVirtualTexture()->GetMaterialType() == ERuntimeVirtualTextureMaterialType::WorldHeight &&
		VirtualHeightfieldMesh::IsEnabled(GetScene() ? GetScene()->GetFeatureLevel() : ERHIFeatureLevel::SM5);
}

FBoxSphereBounds UVirtualHeightfieldMeshComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	return FBoxSphereBounds(FBox(FVector(0.f, 0.f, 0.f), FVector(1.f, 1.f, 1.f))).TransformBy(LocalToWorld);
}

FPrimitiveSceneProxy* UVirtualHeightfieldMeshComponent::CreateSceneProxy()
{
	const FStaticFeatureLevel FeatureLevel = GetScene() ? GetScene()->GetFeatureLevel() : ERHIFeatureLevel::SM5;
	const bool bIsEnabled = VirtualHeightfieldMesh::IsEnabled(FeatureLevel);
	return bIsEnabled ? new FVirtualHeightfieldMeshSceneProxy(this) : nullptr;
}

void UVirtualHeightfieldMeshComponent::SetMaterial(int32 InElementIndex, UMaterialInterface* InMaterial)
{
	if (InElementIndex == 0 && Material != InMaterial)
	{
		Material = InMaterial;
		MarkRenderStateDirty();
	}
}

void UVirtualHeightfieldMeshComponent::GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials) const
{
	if (Material != nullptr)
	{
		OutMaterials.Add(Material);
	}
}

void UVirtualHeightfieldMeshComponent::GatherHideFlags(bool& InOutHidePrimitivesInEditor, bool& InOutHidePrimitivesInGame) const
{
	const FStaticFeatureLevel FeatureLevel = GetScene() ? GetScene()->GetFeatureLevel() : ERHIFeatureLevel::SM5;
	const bool bIsEnabled = VirtualHeightfieldMesh::IsEnabled(FeatureLevel);
	InOutHidePrimitivesInEditor |= (bIsEnabled && !bHiddenInEditor);
	InOutHidePrimitivesInGame |= bIsEnabled;
}

void UVirtualHeightfieldMeshComponent::OnVirtualTextureTransformUpdate(USceneComponent* InRootComponent, EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport)
{
	MarkRenderStateDirty();
}

#if WITH_EDITOR

void UVirtualHeightfieldMeshComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	static const FName HideInEditorName = GET_MEMBER_NAME_CHECKED(UVirtualHeightfieldMeshComponent, bHiddenInEditor);

	const FName PropertyName = PropertyChangedEvent.Property->GetFName();
	if (PropertyName == HideInEditorName)
	{
		// Force RuntimeVirtualTextureComponent to poll the HidePrimitives settings.
		URuntimeVirtualTextureComponent* RuntimeVirtualTextureComponent = VirtualTextureRef != nullptr ? ToRawPtr(VirtualTextureRef->VirtualTextureComponent) : nullptr;
		if (RuntimeVirtualTextureComponent != nullptr)
		{
			RuntimeVirtualTextureComponent->MarkRenderStateDirty();
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#endif

bool UVirtualHeightfieldMeshComponent::IsMinMaxTextureEnabled() const
{
	URuntimeVirtualTexture* RuntimeVirtualTexture = GetVirtualTexture();
	return RuntimeVirtualTexture != nullptr && RuntimeVirtualTexture->GetMaterialType() == ERuntimeVirtualTextureMaterialType::WorldHeight;
}

#if WITH_EDITOR

void UVirtualHeightfieldMeshComponent::InitializeMinMaxTexture(uint32 InSizeX, uint32 InSizeY, uint32 InNumMips, uint8* InData)
{
	// We need an existing StreamingTexture object to update.
	if (MinMaxTexture != nullptr)
	{
		FHeightfieldMinMaxTextureBuildDesc BuildDesc;
		BuildDesc.SizeX = InSizeX;
		BuildDesc.SizeY = InSizeY;
		BuildDesc.NumMips = InNumMips;
		BuildDesc.Data = InData;

		MinMaxTexture->Modify();
		MinMaxTexture->BuildTexture(BuildDesc);

		MarkRenderStateDirty();
	}
}

#endif

