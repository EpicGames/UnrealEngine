// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualHeightfieldMeshEnable.h"

#include "Components/RuntimeVirtualTextureComponent.h"
#include "RHIGlobals.h"
#include "UObject/UObjectIterator.h"
#include "VirtualHeightfieldMeshComponent.h"
#include "VT/RuntimeVirtualTextureVolume.h"
#include "RenderUtils.h"

namespace VirtualHeightfieldMesh
{
	/** CVar to toggle support for virtual heightfield mesh. */
	static TAutoConsoleVariable<int32> CVarVHMEnable(
		TEXT("r.VHM.Enable"),
		1,
		TEXT("Enable virtual heightfield mesh"),
		ECVF_RenderThreadSafe
	);

	/** Sink to apply updates when virtual heightfield mesh settings change. */
	static void OnUpdate()
	{
		const bool bEnable = CVarVHMEnable.GetValueOnGameThread() != 0;

		static bool bLastEnable = !bEnable;

		if (bEnable != bLastEnable)
		{
			bLastEnable = bEnable;

			TArray<URuntimeVirtualTexture*> RuntimeVirtualTextures;

			for (TObjectIterator<UVirtualHeightfieldMeshComponent> It; It; ++It)
			{
				It->MarkRenderStateDirty();

				ARuntimeVirtualTextureVolume* VirtualTextureVolume = It->GetVirtualTextureVolume();
				URuntimeVirtualTextureComponent* VirtualTextureComponent = VirtualTextureVolume != nullptr ? ToRawPtr(VirtualTextureVolume->VirtualTextureComponent) : nullptr;
				URuntimeVirtualTexture* VirtualTexture = VirtualTextureComponent != nullptr ? VirtualTextureComponent->GetVirtualTexture() : nullptr;

				if (VirtualTextureComponent != nullptr)
				{
					VirtualTextureComponent->MarkRenderStateDirty();
				}
				if (VirtualTexture != nullptr)
				{
					RuntimeVirtualTextures.AddUnique(It->GetVirtualTexture());
				}
			}

			for (TObjectIterator<UPrimitiveComponent> It; It; ++It)
			{
				for (URuntimeVirtualTexture* RuntimeVirtualTexture : RuntimeVirtualTextures)
				{
					if (It->GetRuntimeVirtualTextures().Contains(RuntimeVirtualTexture))
					{
						It->MarkRenderStateDirty();
						break;
					}
				}
			}
		}
	}

	FAutoConsoleVariableSink GConsoleVariableSink(FConsoleCommandDelegate::CreateStatic(&OnUpdate));

	bool IsEnabled(FStaticFeatureLevel InFeatureLevel)
	{
		return CVarVHMEnable.GetValueOnAnyThread() != 0 && (InFeatureLevel >= ERHIFeatureLevel::SM5) && UseVirtualTexturing(GetFeatureLevelShaderPlatform(InFeatureLevel));
	}
}
