// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHIFeatureLevel.h"

namespace VirtualHeightfieldMesh
{
	/** Returns true if VirtualHeightfieldMesh is enabled. */
	bool IsEnabled(FStaticFeatureLevel InFeatureLevel);
}
