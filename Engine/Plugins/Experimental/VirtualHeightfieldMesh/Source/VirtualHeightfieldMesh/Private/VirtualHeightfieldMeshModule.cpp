// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualHeightfieldMeshModule.h"

#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "ShaderCore.h"
#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FVirtualHeightfieldMeshModule, VirtualHeightfieldMesh);

void FVirtualHeightfieldMeshModule::StartupModule()
{
	FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("VirtualHeightfieldMesh"))->GetBaseDir(), TEXT("Shaders"));
	AddShaderSourceDirectoryMapping(TEXT("/Plugin/VirtualHeightfieldMesh"), PluginShaderDir);
}

void FVirtualHeightfieldMeshModule::ShutdownModule()
{
}
