// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualHeightfieldMeshSceneProxy.h"

#include "DataDrivenShaderPlatformInfo.h"
#include "EngineModule.h"
#include "Engine/Engine.h"
#include "Engine/Texture2D.h"
#include "GlobalRenderResources.h"
#include "GlobalShader.h"
#include "HeightfieldMinMaxTexture.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Materials/MaterialRenderProxy.h"
#include "PrimitiveViewRelevance.h"
#include "RHIStaticStates.h"
#include "RenderGraphUtils.h"
#include "SceneInterface.h"
#include "SceneView.h"
#include "TextureResource.h"
#include "VirtualHeightfieldMeshComponent.h"
#include "VirtualHeightfieldMeshVertexFactory.h"
#include "VT/RuntimeVirtualTexture.h"
#include "VT/VirtualTextureFeedbackBuffer.h"
#include "Async/Mutex.h"

DECLARE_STATS_GROUP(TEXT("VirtualHeightfieldMesh"), STATGROUP_VirtualHeightfieldMesh, STATCAT_Advanced);
DECLARE_CYCLE_STAT(TEXT("VirtualHeightfieldMesh SubmitWork"), STAT_VirtualHeightfieldMesh_SubmitWork, STATGROUP_VirtualHeightfieldMesh);

DECLARE_GPU_STAT(VirtualHeightfieldMesh);

DECLARE_LOG_CATEGORY_EXTERN(LogVirtualHeightfieldMesh, Warning, All);
DEFINE_LOG_CATEGORY(LogVirtualHeightfieldMesh);

static TAutoConsoleVariable<float> CVarVHMLodScale(
	TEXT("r.VHM.LodScale"),
	1.f,
	TEXT("Global LOD scale applied for Virtual Heightfield Mesh."),
	ECVF_RenderThreadSafe
);

// We disable View.LODDistanceFactor by default.
// When it is set according to GCalcLocalPlayerCachedLODDistanceFactor in ULocalPlayer we end up with double couting of the FOV scale.
// Ideally we would remove the calculation in ULocalPlayer and View.LODDistanceFactor would be only for view specific adjustments (screen captures etc.)
// However the removal of the code in ULocalPlayer could have a big impact on any preexisting data in any project.
static TAutoConsoleVariable<int32> CVarVHMEnableViewLodFactor(
	TEXT("r.VHM.EnableViewLodFactor"),
	0,
	TEXT("Enable the View.LODDistanceFactor.")
	TEXT("This is disabled by default to avoid an issue where FOV is double counted when calculating Lods.")
	TEXT("See comment in code for more information."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarVHMOcclusion(
	TEXT("r.VHM.Occlusion"),
	1,
	TEXT("Enable occlusion queries."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarVHMMaxRenderItems(
	TEXT("r.VHM.MaxRenderInstances"),
	1024 * 4,
	TEXT("Size of buffers used to collect render instances."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarVHMMaxFeedbackItems(
	TEXT("r.VHM.MaxFeedbackItems"),
	1024,
	TEXT("Size of buffer used by virtual texture feedback."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarVHMMaxPersistentQueueItems(
	TEXT("r.VHM.MaxPersistentQueueItems"),
	1024 * 4,
	TEXT("Size of queue used in the collect pass. This is rounded to the nearest power of 2."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarVHMCollectPassWavefronts(
	TEXT("r.VHM.CollectPassWavefronts"),
	16,
	TEXT("Number of wavefronts to use for collect pass."),
	ECVF_RenderThreadSafe
);

namespace VirtualHeightfieldMesh
{
	/** Buffers filled by GPU culling used by the Virtual Heightfield Mesh final draw call. */
	struct FDrawInstanceBuffers
	{
		/* Culled instance buffer. */
		FBufferRHIRef InstanceBuffer;
		FUnorderedAccessViewRHIRef InstanceBufferUAV;
		FShaderResourceViewRHIRef InstanceBufferSRV;

		/* IndirectArgs buffer for final DrawInstancedIndirect. */
		FBufferRHIRef IndirectArgsBuffer;
		FUnorderedAccessViewRHIRef IndirectArgsBufferUAV;
	};

	/** Initialize the FDrawInstanceBuffers objects. */
	void InitializeInstanceBuffers(FRHICommandListBase& InRHICmdList, FDrawInstanceBuffers& InBuffers);

	/** Release the FDrawInstanceBuffers objects. */
	void ReleaseInstanceBuffers(FDrawInstanceBuffers& InBuffers)
	{
		InBuffers.InstanceBuffer.SafeRelease();
		InBuffers.InstanceBufferUAV.SafeRelease();
		InBuffers.InstanceBufferSRV.SafeRelease();
		InBuffers.IndirectArgsBuffer.SafeRelease();
		InBuffers.IndirectArgsBufferUAV.SafeRelease();
	}
}

struct FOcclusionResults
{
	FTextureRHIRef OcclusionTexture;
	FIntPoint TextureSize;
	int32 NumTextureMips;
	TArray<bool> UploadData;
};

struct FOcclusionResultsKey
{
	FVirtualHeightfieldMeshSceneProxy const* Proxy;
	FSceneView const* View;

	FOcclusionResultsKey(FVirtualHeightfieldMeshSceneProxy const* InProxy, FSceneView const* InView)
		: Proxy(InProxy)
		, View(InView)
	{
	}

	friend inline uint32 GetTypeHash(const FOcclusionResultsKey& InKey)
	{
		return HashCombine(GetTypeHash(InKey.View), GetTypeHash(InKey.Proxy));
	}

	friend bool operator==(const FOcclusionResultsKey& A, const FOcclusionResultsKey& B)
	{
		return A.View == B.View && A.Proxy == B.Proxy;
	}
};

/** Global map for occlusion result. */
TMap< FOcclusionResultsKey, FOcclusionResults > GOcclusionResults;
bool GOcclusionResetRequired = false;

namespace VirtualHeightfieldMesh
{
	/** Calculate distances used for LODs in a given view for a given scene proxy. */
	FVector4f CalculateLodRanges(FSceneView const* InView, FVirtualHeightfieldMeshSceneProxy const* InProxy)
	{
		const uint32 MaxLevel = InProxy->AllocatedVirtualTexture->GetMaxLevel();
		const float Lod0UVSize = 1.f / (float)(1 << MaxLevel);
		const FVector2D Lod0WorldSize = FVector2D(InProxy->UVToWorldScale.X, InProxy->UVToWorldScale.Y) * Lod0UVSize; // LWC_TODO: precision loss
		const float Lod0WorldRadius = Lod0WorldSize.Size();
		const float ScreenMultiple = FMath::Max(0.5f * InView->ViewMatrices.GetViewToClip().M[0][0], 0.5f * InView->ViewMatrices.GetViewToClip().M[1][1]);
		const float Lod0Distance = Lod0WorldRadius * ScreenMultiple / InProxy->Lod0ScreenSize;
		const float ViewLodDistanceFactor = CVarVHMEnableViewLodFactor.GetValueOnRenderThread() == 0 ? 1.f : InView->LODDistanceFactor;
		const float LodScale = ViewLodDistanceFactor * CVarVHMLodScale.GetValueOnRenderThread();
		
		return FVector4f(Lod0Distance, InProxy->Lod0Distribution, InProxy->LodDistribution, LodScale);
	}
}

/** Renderer extension to manage the buffer pool and add hooks for GPU culling passes. */
class FVirtualHeightfieldMeshRendererExtension : public FRenderResource
{
public:
	FVirtualHeightfieldMeshRendererExtension()
		: bInFrame(false)
		, DiscardId(0)
	{}

	virtual ~FVirtualHeightfieldMeshRendererExtension()
	{}

	/** Call once to register this extension. */
	void RegisterExtension();

	/** Are we inside a BeginFrame()/EndFrame() scope? */
	bool IsInFrame() { return bInFrame; }

	/** Call once per frame for each mesh/view that has relevance. This allocates the buffers to use for the frame and adds the work to fill the buffers to the queue. */
	VirtualHeightfieldMesh::FDrawInstanceBuffers& AddWork(FRHICommandListBase& RHICmdList, FVirtualHeightfieldMeshSceneProxy const* InProxy, FSceneView const* InMainView, FSceneView const* InCullView);
	/** Submit all the work added by AddWork(). The work fills all of the buffers ready for use by the referencing mesh batches. */
	void SubmitWork(FRDGBuilder& GraphBuilder);

protected:
	//~ Begin FRenderResource Interface
	virtual void ReleaseRHI() override;
	//~ End FRenderResource Interface

private:
	/** Called by renderer at start of render frame. */
	void BeginFrame(FRDGBuilder& GraphBuilder);
	/** Called by renderer at end of render frame. */
	void EndFrame(FRDGBuilder& GraphBuilder);
	void EndFrame();

	/** Flag for frame validation. */
	bool bInFrame;

	UE::FRecursiveMutex Mutex;

	/** Buffers to fill. Resources can persist between frames to reduce allocation cost, but contents don't persist. */
	TArray<VirtualHeightfieldMesh::FDrawInstanceBuffers> Buffers;
	/** Per buffer frame time stamp of last usage. */
	TArray<uint32> DiscardIds;
	/** Current frame time stamp. */
	uint32 DiscardId;

	/** Arrary of uniqe scene proxies to render this frame. */
	TArray<FVirtualHeightfieldMeshSceneProxy const*> SceneProxies;
	/** Arrary of unique main views to render this frame. */
	TArray<FSceneView const*> MainViews;
	/** Arrary of unique culling views to render this frame. */
	TArray<FSceneView const*> CullViews;

	/** Key for each buffer we need to generate. */
	struct FWorkDesc
	{
		int32 ProxyIndex;
		int32 MainViewIndex;
		int32 CullViewIndex;
		int32 BufferIndex;
	};

	/** Keys specifying what to render. */
	TArray<FWorkDesc> WorkDescs;

	/** Sort predicate for FWorkDesc. When rendering we want to batch work by proxy, then by main view. */
	struct FWorkDescSort
	{
		uint32 SortKey(FWorkDesc const& WorkDesc) const
		{
			return (WorkDesc.ProxyIndex << 24) | (WorkDesc.MainViewIndex << 16) | (WorkDesc.CullViewIndex << 8) | WorkDesc.BufferIndex;
		}

		bool operator()(FWorkDesc const& A, FWorkDesc const& B) const
		{
			return SortKey(A) < SortKey(B);
		}
	};
};

/** Single global instance of the VirtualHeightfieldMesh renderer extension. */
TGlobalResource< FVirtualHeightfieldMeshRendererExtension > GVirtualHeightfieldMeshViewRendererExtension;

void FVirtualHeightfieldMeshRendererExtension::RegisterExtension()
{
	static bool bInit = false;
	if (!bInit)
	{
		GEngine->GetPreRenderDelegateEx().AddRaw(this, &FVirtualHeightfieldMeshRendererExtension::BeginFrame);
		GEngine->GetPostRenderDelegateEx().AddRaw(this, &FVirtualHeightfieldMeshRendererExtension::EndFrame);
		bInit = true;
	}
}

void FVirtualHeightfieldMeshRendererExtension::ReleaseRHI()
{
	Buffers.Empty();
}

VirtualHeightfieldMesh::FDrawInstanceBuffers& FVirtualHeightfieldMeshRendererExtension::AddWork(FRHICommandListBase& RHICmdList, FVirtualHeightfieldMeshSceneProxy const* InProxy, FSceneView const* InMainView, FSceneView const* InCullView)
{
	UE::TScopeLock Lock(Mutex);

	// If we hit this then BegineFrame()/EndFrame() logic needs fixing in the Scene Renderer.
	if (!ensure(!bInFrame))
	{
		EndFrame();
	}

	// Create workload
	FWorkDesc WorkDesc;
	WorkDesc.ProxyIndex = SceneProxies.AddUnique(InProxy);
	WorkDesc.MainViewIndex = MainViews.AddUnique(InMainView);
	WorkDesc.CullViewIndex = CullViews.AddUnique(InCullView);
	WorkDesc.BufferIndex = -1;

	// Check for an existing duplicate
	for (FWorkDesc& It : WorkDescs)
	{
		if (It.ProxyIndex == WorkDesc.ProxyIndex && It.MainViewIndex == WorkDesc.MainViewIndex && It.CullViewIndex == WorkDesc.CullViewIndex && It.BufferIndex != -1)
		{
			WorkDesc.BufferIndex = It.BufferIndex;
			break;
		}
	}

	// Try to recycle a buffer
	if (WorkDesc.BufferIndex == -1)
	{
		for (int32 BufferIndex = 0; BufferIndex < Buffers.Num(); BufferIndex++)
		{
			if (DiscardIds[BufferIndex] < DiscardId)
			{
				DiscardIds[BufferIndex] = DiscardId;
				WorkDesc.BufferIndex = BufferIndex;
				WorkDescs.Add(WorkDesc);
				break;
			}
		}
	}

	// Allocate new buffer if necessary
	if (WorkDesc.BufferIndex == -1)
	{
		DiscardIds.Add(DiscardId);
		WorkDesc.BufferIndex = Buffers.AddDefaulted();
		WorkDescs.Add(WorkDesc);
		VirtualHeightfieldMesh::InitializeInstanceBuffers(RHICmdList, Buffers[WorkDesc.BufferIndex]);
	}

	return Buffers[WorkDesc.BufferIndex];
}

void FVirtualHeightfieldMeshRendererExtension::BeginFrame(FRDGBuilder& GraphBuilder)
{
	UE::TScopeLock Lock(Mutex);
	// If we hit this then BegineFrame()/EndFrame() logic needs fixing in the Scene Renderer.
	if (!ensure(!bInFrame))
	{
		EndFrame();
	}
	bInFrame = true;

	if (WorkDescs.Num() > 0)
	{
		SubmitWork(GraphBuilder);
	}
}

void FVirtualHeightfieldMeshRendererExtension::EndFrame()
{
	UE::TScopeLock Lock(Mutex);
	ensure(bInFrame);
	bInFrame = false;

	SceneProxies.Reset();
	MainViews.Reset();
	CullViews.Reset();
	WorkDescs.Reset();

	// Clean the buffer pool
	DiscardId++;

	for (int32 Index = 0; Index < DiscardIds.Num();)
	{
		if (DiscardId - DiscardIds[Index] > 4u)
		{
			VirtualHeightfieldMesh::ReleaseInstanceBuffers(Buffers[Index]);
			Buffers.RemoveAtSwap(Index);
			DiscardIds.RemoveAtSwap(Index);
		}
		else
		{
			++Index;
		}
	}

	GOcclusionResetRequired = true;
}

void FVirtualHeightfieldMeshRendererExtension::EndFrame(FRDGBuilder& GraphBuilder)
{
	EndFrame();
}

const static FName NAME_VirtualHeightfieldMesh(TEXT("VirtualHeightfieldMesh"));

FVirtualHeightfieldMeshSceneProxy::FVirtualHeightfieldMeshSceneProxy(UVirtualHeightfieldMeshComponent* InComponent)
	: FPrimitiveSceneProxy(InComponent, NAME_VirtualHeightfieldMesh)
	, bHiddenInEditor(InComponent->GetHiddenInEditor())
	, RuntimeVirtualTexture(InComponent->GetVirtualTexture())
	, HeightMinMaxTexture(nullptr)
	, LodBiasTexture(nullptr)
	, LodBiasMinMaxTexture(nullptr)
	, AllocatedVirtualTexture(nullptr)
	, bCallbackRegistered(false)
	, NumQuadsPerTileSide(0)
	, VertexFactory(nullptr)
	, Lod0ScreenSize(InComponent->GetLod0ScreenSize())
	, Lod0Distribution(InComponent->GetLod0Distribution())
	, LodDistribution(InComponent->GetLodDistribution())
	, LodBiasScale(InComponent->GetLodBiasScale())
	, NumForceLoadLods(InComponent->GetNumForceLoadLods())
	, NumOcclusionLods(0)
	, OcclusionGridSize(0, 0)
{
	GVirtualHeightfieldMeshViewRendererExtension.RegisterExtension();

	// They have some LOD, but considered static as the LODs (are intended to) represent the same static surface.
	bHasDeformableMesh = false;

	UMaterialInterface* ComponentMaterial = InComponent->GetMaterial();
	const bool bValidMaterial = ComponentMaterial != nullptr && ComponentMaterial->CheckMaterialUsage_Concurrent(MATUSAGE_VirtualHeightfieldMesh);
	Material = bValidMaterial ? ComponentMaterial->GetRenderProxy() : UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy();
	MaterialRelevance = Material->GetMaterialInterface()->GetRelevance_Concurrent(GetScene().GetShaderPlatform());

	const FTransform VirtualTextureTransform = InComponent->GetVirtualTextureTransform();

	UVToWorldScale = VirtualTextureTransform.GetScale3D();
	UVToWorld = VirtualTextureTransform.ToMatrixWithScale();

	WorldToUV = UVToWorld.Inverse();
	WorldToUVTransposeAdjoint = WorldToUV.TransposeAdjoint();

	// UVToLocal will be initialized in OnTransformChanged() called immediately after construction.
	UVToLocal = FMatrix::Identity;

	UHeightfieldMinMaxTexture* HeightfieldMinMaxTexture = InComponent->GetMinMaxTexture();
	if (HeightfieldMinMaxTexture != nullptr)
	{
		HeightMinMaxTexture = HeightfieldMinMaxTexture->Texture;
		BuildOcclusionVolumes(HeightfieldMinMaxTexture->TextureData, HeightfieldMinMaxTexture->TextureDataSize, HeightfieldMinMaxTexture->TextureDataMips, InComponent->GetNumOcclusionLods());

		LodBiasTexture = HeightfieldMinMaxTexture->LodBiasTexture;
		LodBiasMinMaxTexture = HeightfieldMinMaxTexture->LodBiasMinMaxTexture;
	}
}

SIZE_T FVirtualHeightfieldMeshSceneProxy::GetTypeHash() const
{
	static size_t UniquePointer;
	return reinterpret_cast<size_t>(&UniquePointer);
}

uint32 FVirtualHeightfieldMeshSceneProxy::GetMemoryFootprint() const
{
	return(sizeof(*this) + FPrimitiveSceneProxy::GetAllocatedSize());
}

void FVirtualHeightfieldMeshSceneProxy::OnTransformChanged(FRHICommandListBase& RHICmdList)
{
	UVToLocal = UVToWorld * GetLocalToWorld().Inverse();

	// Setup a default occlusion volume array containing just the primitive bounds.
	// We use this if disabling the full set of occlusion volumes.
	DefaultOcclusionVolumes.Reset();
	DefaultOcclusionVolumes.Add(GetBounds());
}

void FVirtualHeightfieldMeshSceneProxy::CreateRenderThreadResources(FRHICommandListBase& RHICmdList)
{
	if (RuntimeVirtualTexture != nullptr)
	{
		if (!bCallbackRegistered)
		{
			GetRendererModule().AddVirtualTextureProducerDestroyedCallback(RuntimeVirtualTexture->GetProducerHandle(), &OnVirtualTextureDestroyedCB, this);
			bCallbackRegistered = true;
		}

		if (RuntimeVirtualTexture->GetMaterialType() == ERuntimeVirtualTextureMaterialType::WorldHeight)
		{
			AllocatedVirtualTexture = RuntimeVirtualTexture->GetAllocatedVirtualTexture();
			NumQuadsPerTileSide = RuntimeVirtualTexture->GetTileSize();

			if (AllocatedVirtualTexture != nullptr)
			{
				// Gather vertex factory uniform parameters.
				FVirtualHeightfieldMeshVertexFactoryParameters UniformParams;
				UniformParams.PageTableTexture = AllocatedVirtualTexture->GetPageTableTexture(0);
				UniformParams.HeightTexture = AllocatedVirtualTexture->GetPhysicalTextureSRV(0, false);
				UniformParams.HeightSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
				UniformParams.LodBiasTexture = LodBiasTexture ? LodBiasTexture->GetResource()->TextureRHI : GBlackTexture->TextureRHI;
				UniformParams.LodBiasSampler = TStaticSamplerState<SF_Point>::GetRHI();
				UniformParams.NumQuadsPerTileSide = NumQuadsPerTileSide;

				FUintVector4 PackedUniform;
				AllocatedVirtualTexture->GetPackedUniform(&PackedUniform, 0);
				UniformParams.VTPackedUniform = PackedUniform;
				FUintVector4 PackedPageTableUniform[2];
				AllocatedVirtualTexture->GetPackedPageTableUniform(PackedPageTableUniform);
				UniformParams.VTPackedPageTableUniform0 = PackedPageTableUniform[0];
				UniformParams.VTPackedPageTableUniform1 = PackedPageTableUniform[1];

				const float PageTableSizeX = AllocatedVirtualTexture->GetWidthInTiles();
				const float PageTableSizeY = AllocatedVirtualTexture->GetHeightInTiles();
				UniformParams.PageTableSize = FVector4f(PageTableSizeX, PageTableSizeY, 1.f / PageTableSizeX, 1.f / PageTableSizeY);

				const float PhysicalTextureSize = AllocatedVirtualTexture->GetPhysicalTextureSize(0);
				UniformParams.PhysicalTextureSize = FVector2f(PhysicalTextureSize, 1.f / PhysicalTextureSize);

				UniformParams.VirtualHeightfieldToLocal = FMatrix44f(UVToLocal);
				UniformParams.VirtualHeightfieldToWorld = FMatrix44f(UVToWorld);		// LWC_TODO: Precision loss

				UniformParams.MaxLod = AllocatedVirtualTexture->GetMaxLevel();
				UniformParams.LodBiasScale = LodBiasScale;

				// Create vertex factory.
				VertexFactory = new FVirtualHeightfieldMeshVertexFactory(GetScene().GetFeatureLevel(), UniformParams);
				VertexFactory->InitResource(RHICmdList);
			}
		}
	}
}

void FVirtualHeightfieldMeshSceneProxy::DestroyRenderThreadResources()
{
	if (VertexFactory != nullptr)
	{
		VertexFactory->ReleaseResource();
		delete VertexFactory;
		VertexFactory = nullptr;
	}

	if (bCallbackRegistered)
	{
		GetRendererModule().RemoveAllVirtualTextureProducerDestroyedCallbacks(this);
		bCallbackRegistered = false;
	}
}

void FVirtualHeightfieldMeshSceneProxy::OnVirtualTextureDestroyedCB(const FVirtualTextureProducerHandle& InHandle, void* Baton)
{
	FVirtualHeightfieldMeshSceneProxy* SceneProxy = (FVirtualHeightfieldMeshSceneProxy*)Baton;
	SceneProxy->DestroyRenderThreadResources();
	SceneProxy->CreateRenderThreadResources(FRHICommandListImmediate::Get());
}

FPrimitiveViewRelevance FVirtualHeightfieldMeshSceneProxy::GetViewRelevance(const FSceneView* View) const
{
	const bool bValid = AllocatedVirtualTexture != nullptr;
	const bool bIsHiddenInEditor = bHiddenInEditor && View->Family->EngineShowFlags.Editor;

	FPrimitiveViewRelevance Result;
	Result.bDrawRelevance = bValid && IsShown(View) && !bIsHiddenInEditor;
	Result.bShadowRelevance = bValid && IsShadowCast(View) && ShouldRenderInMainPass() &&!bIsHiddenInEditor;
	Result.bDynamicRelevance = true;
	Result.bStaticRelevance = false;
	Result.bRenderInMainPass = ShouldRenderInMainPass();
	Result.bUsesLightingChannels = GetLightingChannelMask() != GetDefaultLightingChannelMask();
	Result.bRenderCustomDepth = ShouldRenderCustomDepth();
	Result.bTranslucentSelfShadow = false;
	MaterialRelevance.SetPrimitiveViewRelevance(Result);
	Result.bVelocityRelevance = DrawsVelocity() && Result.bOpaque && Result.bRenderInMainPass;
	return Result;
}

void FVirtualHeightfieldMeshSceneProxy::GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const
{
	check(AllocatedVirtualTexture != nullptr);

	if (GVirtualHeightfieldMeshViewRendererExtension.IsInFrame())
	{
		// Can't add new work while bInFrame.
		// In UE5 we need to AddWork()/SubmitWork() in two phases: InitViews() and InitViewsAfterPrepass()
		// The main renderer hooks for that don't exist in UE5.0 and are only added in UE5.1
		// That means that for UE5.0 we always hit this for shadow drawing and shadows will not be rendered.
		// Not earlying out here can lead to crashes from buffers being released too soon.
		return;
	}

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		if (VisibilityMap & (1 << ViewIndex))
		{
			VirtualHeightfieldMesh::FDrawInstanceBuffers& Buffers = GVirtualHeightfieldMeshViewRendererExtension.AddWork(Collector.GetRHICommandList(), this, ViewFamily.Views[0], Views[ViewIndex]);

			FMeshBatch& Mesh = Collector.AllocateMesh();
			Mesh.bWireframe = AllowDebugViewmodes() && ViewFamily.EngineShowFlags.Wireframe;
			Mesh.bUseWireframeSelectionColoring = IsSelected();
			Mesh.VertexFactory = VertexFactory;
			Mesh.MaterialRenderProxy = Material;
			Mesh.ReverseCulling = IsLocalToWorldDeterminantNegative();
			Mesh.Type = PT_TriangleList;
			Mesh.DepthPriorityGroup = SDPG_World;
			Mesh.bCanApplyViewModeOverrides = true;
			Mesh.bUseForMaterial = true;
			Mesh.CastShadow = true;
			Mesh.bUseForDepthPass = true;

			Mesh.Elements.SetNumZeroed(1);
			{
				FMeshBatchElement& BatchElement = Mesh.Elements[0];

				BatchElement.IndexBuffer = VertexFactory->GetIndexBuffer();
				BatchElement.IndirectArgsBuffer = Buffers.IndirectArgsBuffer;
				BatchElement.IndirectArgsOffset = 0;

				BatchElement.FirstIndex = 0;
				BatchElement.NumPrimitives = 0;
				BatchElement.MinVertexIndex = 0;
				BatchElement.MaxVertexIndex = 0;

				BatchElement.PrimitiveIdMode = PrimID_ForceZero;
				BatchElement.PrimitiveUniformBuffer = GetUniformBuffer();

				FVirtualHeightfieldMeshUserData* UserData = &Collector.AllocateOneFrameResource<FVirtualHeightfieldMeshUserData>();
				BatchElement.UserData = (void*)UserData;

				UserData->InstanceBufferSRV = Buffers.InstanceBufferSRV;

				//todo[vhm]: Move all the view dependent lod logic into shader. Would help us to move to static mesh batches in the future.
				FSceneView const* MainView = ViewFamily.Views[0];
				UserData->LodViewOrigin = (FVector3f)MainView->ViewMatrices.GetViewOrigin();	// LWC_TODO: Precision Loss

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				// Support the freezerendering mode. Use any frozen view state for culling.
				const FViewMatrices* FrozenViewMatrices = MainView->State != nullptr ? MainView->State->GetFrozenViewMatrices() : nullptr;
				if (FrozenViewMatrices != nullptr)
				{
					UserData->LodViewOrigin = (FVector3f)FrozenViewMatrices->GetViewOrigin();
				}
#endif
				
				UserData->LodDistances = VirtualHeightfieldMesh::CalculateLodRanges(MainView, this);
			}

			Collector.AddMesh(ViewIndex, Mesh);
		}
	}
}

bool FVirtualHeightfieldMeshSceneProxy::HasSubprimitiveOcclusionQueries() const
{
	return CVarVHMOcclusion.GetValueOnAnyThread() != 0;
}

const TArray<FBoxSphereBounds>* FVirtualHeightfieldMeshSceneProxy::GetOcclusionQueries(const FSceneView* View) const
{
	return (CVarVHMOcclusion.GetValueOnAnyThread() == 0 || OcclusionVolumes.Num() == 0) ? &DefaultOcclusionVolumes : &OcclusionVolumes;
}

void FVirtualHeightfieldMeshSceneProxy::BuildOcclusionVolumes(TArrayView<FVector2D> const& InMinMaxData, FIntPoint const& InMinMaxSize, TArrayView<int32> const& InMinMaxMips, int32 InNumLods)
{
	NumOcclusionLods = 0;
	OcclusionGridSize = FIntPoint::ZeroValue;
	OcclusionVolumes.Reset();

	if (InNumLods > 0 && InMinMaxMips.Num() > 0)
	{
		NumOcclusionLods = FMath::Min(InNumLods, InMinMaxMips.Num());

		const int32 BaseLod = InMinMaxMips.Num() - NumOcclusionLods;
		OcclusionGridSize.X = FMath::Max(InMinMaxSize.X >> BaseLod, 1);
		OcclusionGridSize.Y = FMath::Max(InMinMaxSize.Y >> BaseLod, 1);

		OcclusionVolumes.Reserve(InMinMaxData.Num() - InMinMaxMips[BaseLod]);

		for (int32 LodIndex = BaseLod; LodIndex < InMinMaxMips.Num(); ++LodIndex)
		{
			int32 SizeX = FMath::Max(InMinMaxSize.X >> LodIndex, 1);
			int32 SizeY = FMath::Max(InMinMaxSize.Y >> LodIndex, 1);
			int32 MinMaxDataIndex = InMinMaxMips[LodIndex];

			for (int Y = 0; Y < SizeY; ++Y)
			{
				for (int X = 0; X < SizeX; ++X)
				{
					FVector2D MinMaxU = FVector2D((float)X / (float)SizeX, (float)(X + 1) / (float)SizeX);
					FVector2D MinMaxV = FVector2D((float)Y / (float)SizeY, (float)(Y + 1) / (float)SizeY);
					FVector2D MinMaxZ = InMinMaxData[MinMaxDataIndex++];

					FVector Pos[8];
					Pos[0] = UVToWorld.TransformPosition(FVector(MinMaxU.X, MinMaxV.X, MinMaxZ.X));
					Pos[1] = UVToWorld.TransformPosition(FVector(MinMaxU.Y, MinMaxV.X, MinMaxZ.X));
					Pos[2] = UVToWorld.TransformPosition(FVector(MinMaxU.X, MinMaxV.Y, MinMaxZ.X));
					Pos[3] = UVToWorld.TransformPosition(FVector(MinMaxU.Y, MinMaxV.Y, MinMaxZ.X));
					Pos[4] = UVToWorld.TransformPosition(FVector(MinMaxU.X, MinMaxV.X, MinMaxZ.Y));
					Pos[5] = UVToWorld.TransformPosition(FVector(MinMaxU.Y, MinMaxV.X, MinMaxZ.Y));
					Pos[6] = UVToWorld.TransformPosition(FVector(MinMaxU.X, MinMaxV.Y, MinMaxZ.Y));
					Pos[7] = UVToWorld.TransformPosition(FVector(MinMaxU.Y, MinMaxV.Y, MinMaxZ.Y));

					const float ExpandOcclusion = 3.f;
					OcclusionVolumes.Add(FBoxSphereBounds(FBox(Pos, 8).ExpandBy(ExpandOcclusion)));
				}
			}
		}
	}
}

void FVirtualHeightfieldMeshSceneProxy::AcceptOcclusionResults(FSceneView const* View, TArray<bool>* Results, int32 ResultsStart, int32 NumResults)
{
	if (GOcclusionResetRequired)
	{
		GOcclusionResults.Reset();
		GOcclusionResetRequired = false;
	}

	if (CVarVHMOcclusion.GetValueOnAnyThread() != 0 && Results != nullptr && NumResults > 1)
	{
		FOcclusionResults& OcclusionResults = GOcclusionResults.Emplace(FOcclusionResultsKey(this, View));
		OcclusionResults.TextureSize = OcclusionGridSize;
		OcclusionResults.NumTextureMips = NumOcclusionLods;
		OcclusionResults.UploadData.Append(Results->GetData() + ResultsStart, NumResults);
	}
}

namespace VirtualHeightfieldMesh
{
	/* Keep indirect args offsets in sync with VirtualHeightfieldMesh.usf. */
	static const int32 IndirectArgsByteOffset_FinalCull = 0;
	static const int32 IndirectArgsByteSize = 4 * sizeof(uint32);

	/** Shader structure used for tracking work queues in persistent wave style shaders. Keep in sync with VirtualHeightfieldMesh.ush. */
	struct WorkerQueueInfo
	{
		uint32 Read;
		uint32 Write;
		int32 NumActive;
	};

	/** Final render instance description used by the DrawInstancedIndirect(). Keep in sync with VirtualHeightfieldMesh.ush. */
	struct QuadRenderInstance
	{
		float UVTransform[3];
		uint32 AddressLevelPacked;
	};

	/** Compute shader to initialize all buffers, including adding the lowest mip page(s) to the QuadBuffer. */
	class FInitBuffersVHM_CS : public FGlobalShader
	{
	public:
		DECLARE_GLOBAL_SHADER(FInitBuffersVHM_CS);
		SHADER_USE_PARAMETER_STRUCT(FInitBuffersVHM_CS, FGlobalShader);

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER(uint32, MaxLevel)
			SHADER_PARAMETER(uint32, NumForceLoadLods)
			SHADER_PARAMETER(uint32, PageTableFeedbackId)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<WorkerQueueInfo>, RWQueueInfo)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWQueueBuffer)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint2>, RWQuadBuffer)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWIndirectArgsBuffer)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWFeedbackBuffer)
		END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(FGlobalShaderPermutationParameters const& Parameters)
		{
			return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
		}
	};

	IMPLEMENT_GLOBAL_SHADER(FInitBuffersVHM_CS, "/Plugin/VirtualHeightfieldMesh/Private/VirtualHeightfieldMesh.usf", "InitBuffersCS", SF_Compute);

	/** Compute shader to traverse the virtual texture page table for a view and generate an array of quads to potentially render. */
	class FCollectQuadsVHM_CS : public FGlobalShader
	{
	public:
		DECLARE_GLOBAL_SHADER(FCollectQuadsVHM_CS);
		SHADER_USE_PARAMETER_STRUCT(FCollectQuadsVHM_CS, FGlobalShader);

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_TEXTURE(Texture2D, HeightMinMaxTexture)
			SHADER_PARAMETER_SAMPLER(SamplerState, MinMaxTextureSampler)
			SHADER_PARAMETER(int32, MinMaxLevelOffset)
			SHADER_PARAMETER_TEXTURE(Texture2D, LodBiasMinMaxTexture)
			SHADER_PARAMETER_TEXTURE(Texture2D<float>, OcclusionTexture)
			SHADER_PARAMETER(int32, OcclusionLevelOffset)
			SHADER_PARAMETER_TEXTURE(Texture2D<uint>, PageTableTexture)
			SHADER_PARAMETER(uint32, MaxLevel)
			SHADER_PARAMETER(FVector4f, PageTableSize)
			SHADER_PARAMETER(uint32, PageTableFeedbackId)
			SHADER_PARAMETER(FVector4f, LodDistances)
			SHADER_PARAMETER(float, LodBiasScale)
			SHADER_PARAMETER(FVector3f, ViewOrigin)
			SHADER_PARAMETER_ARRAY(FVector4f, FrustumPlanes, [5])
			SHADER_PARAMETER(FMatrix44f, UVToWorld)
			SHADER_PARAMETER(FVector3f, UVToWorldScale)
			SHADER_PARAMETER(uint32, QueueBufferSizeMask)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<WorkerQueueInfo>, RWQueueInfo)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWQueueBuffer)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint2>, RWQuadBuffer)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWIndirectArgsBuffer)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWFeedbackBuffer)
		END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(FGlobalShaderPermutationParameters const& Parameters)
		{
			return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
		}
	};

	IMPLEMENT_GLOBAL_SHADER(FCollectQuadsVHM_CS, "/Plugin/VirtualHeightfieldMesh/Private/VirtualHeightfieldMesh.usf", "CollectQuadsCS", SF_Compute);

	/** InitInstanceBuffer compute shader. */
	class FInitInstanceBufferVHM_CS : public FGlobalShader
	{
	public:
		DECLARE_GLOBAL_SHADER(FInitInstanceBufferVHM_CS);
		SHADER_USE_PARAMETER_STRUCT(FInitInstanceBufferVHM_CS, FGlobalShader);

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER(int32, NumIndices)
			SHADER_PARAMETER_UAV(RWBuffer<uint>, RWIndirectArgsBuffer)
		END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(FGlobalShaderPermutationParameters const& Parameters)
		{
			return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
		}
	};

	IMPLEMENT_GLOBAL_SHADER(FInitInstanceBufferVHM_CS, "/Plugin/VirtualHeightfieldMesh/Private/VirtualHeightfieldMesh.usf", "InitInstanceBufferCS", SF_Compute);

	/** CullInstances compute shader. */
	class FCullInstancesVHM_CS : public FGlobalShader
	{
	public:
		DECLARE_GLOBAL_SHADER(FCullInstancesVHM_CS);
		SHADER_USE_PARAMETER_STRUCT(FCullInstancesVHM_CS, FGlobalShader);

		class FReuseCullDim : SHADER_PERMUTATION_BOOL("REUSE_CULL");

		using FPermutationDomain = TShaderPermutationDomain<FReuseCullDim>;

		static bool ShouldCompilePermutation(FGlobalShaderPermutationParameters const& Parameters)
		{
			return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
		}

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_TEXTURE(Texture2D, HeightMinMaxTexture)
			SHADER_PARAMETER_SAMPLER(SamplerState, MinMaxTextureSampler)
			SHADER_PARAMETER(int32, MinMaxLevelOffset)
			SHADER_PARAMETER_TEXTURE(Texture2D, PageTableTexture)
			SHADER_PARAMETER(FVector4f, PageTableSize)
			SHADER_PARAMETER_ARRAY(FVector4f, FrustumPlanes, [5])
			SHADER_PARAMETER(FVector4f, PhysicalPageTransform)
			SHADER_PARAMETER(uint32, NumPhysicalAddressBits)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint2>, QuadBuffer)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, IndirectArgsBufferSRV)
			SHADER_PARAMETER_UAV(RWStructuredBuffer<QuadRenderInstance>, RWInstanceBuffer)
			SHADER_PARAMETER_UAV(RWBuffer<uint>, RWIndirectArgsBuffer)
			RDG_BUFFER_ACCESS(IndirectArgsBuffer, ERHIAccess::IndirectArgs)
		END_SHADER_PARAMETER_STRUCT()
	};

	IMPLEMENT_GLOBAL_SHADER(FCullInstancesVHM_CS, "/Plugin/VirtualHeightfieldMesh/Private/VirtualHeightfieldMesh.usf", "CullInstancesCS", SF_Compute);


	/** Default Min/Max texture has the fixed maximum [0,1]. */
	class FHeightMinMaxDefaultTexture : public FTexture
	{
	public:
		virtual void InitRHI(FRHICommandListBase& RHICmdList) override
		{
			const FRHITextureCreateDesc Desc =
				FRHITextureCreateDesc::Create2D(TEXT("VirtualHeightfieldMesh.MinMaxDefaultTexture"), 1, 1, PF_B8G8R8A8)
				.SetFlags(ETextureCreateFlags::ShaderResource)
				.DetermineInititialState()
				.SetInitActionInitializer();

			FRHITextureInitializer Initializer = RHICmdList.CreateTextureInitializer(Desc);
			Initializer.GetTexture2DSubresource(0).WriteColor(FColor(0, 0, 255, 255));
			TextureRHI = Initializer.Finalize();

			// Create the sampler state RHI resource.
			FSamplerStateInitializerRHI SamplerStateInitializer(SF_Point, AM_Clamp, AM_Clamp, AM_Clamp);
			SamplerStateRHI = GetOrCreateSamplerState(SamplerStateInitializer);
		}

		virtual uint32 GetSizeX() const override { return 1; }
		virtual uint32 GetSizeY() const override { return 1; }
	};

	/** Single global instance of default Min/Max texture. */
	FTexture* GHeightMinMaxDefaultTexture = new TGlobalResource<FHeightMinMaxDefaultTexture>;

	/** View matrices that can be frozen in freezerendering mode. */
	struct FViewData
	{
		FVector ViewOrigin;
		FMatrix ProjectionMatrix;
		FConvexVolume ViewFrustum;
		bool bViewFrozen;
	};

	/** Fill the FViewData from an FSceneView respecting the freezerendering mode. */
	void GetViewData(FSceneView const* InSceneView, FViewData& OutViewData)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		const FViewMatrices* FrozenViewMatrices = InSceneView->State != nullptr ? InSceneView->State->GetFrozenViewMatrices() : nullptr;
		if (FrozenViewMatrices != nullptr)
		{
			OutViewData.ViewOrigin = FrozenViewMatrices->GetViewOrigin();
			OutViewData.ProjectionMatrix = FrozenViewMatrices->GetViewToClip();
			GetViewFrustumBounds(OutViewData.ViewFrustum, FrozenViewMatrices->GetWorldToClip(), true);
			OutViewData.bViewFrozen = true;
		}
		else
#endif
		{
			OutViewData.ViewOrigin = InSceneView->ViewMatrices.GetViewOrigin();
			OutViewData.ProjectionMatrix = InSceneView->ViewMatrices.GetViewToClip();
			OutViewData.ViewFrustum = InSceneView->ViewFrustum;
			OutViewData.bViewFrozen = false;
		}
	}

	/** Convert FPlane to Xx+Yy+Zz+W=0 form for simpler use in shader. */
	FVector4 ConvertPlane(FPlane const& Plane)
	{
		return FVector4(-Plane.X, -Plane.Y, -Plane.Z, Plane.W);
	}

	/** Translate a plane. This is a simpler case than the full TransformPlane(). */
	FPlane TranslatePlane(FPlane const& Plane, FVector const& Translation)
	{
		FPlane OutPlane = Plane / Plane.Size();
		OutPlane.W -= FVector::DotProduct(FVector(OutPlane),  Translation);
		return OutPlane;
	}

	/** Transform a plane using a transform matrix. Precalculate and pass in transpose adjoint to avoid work when transforming multiple planes.  */
	FPlane TransformPlane(FPlane const& Plane, FMatrix const& Matrix, FMatrix const& TransposeAdjoint)
	{
		FVector N(Plane.X, Plane.Y, Plane.Z);
		N = TransposeAdjoint.TransformVector(N).GetUnsafeNormal3();

		FVector P(Plane.X * Plane.W, Plane.Y * Plane.W, Plane.Z * Plane.W);
		P = Matrix.TransformPosition(P);

		return FPlane(N, FVector::DotProduct(N, P));
	}

	/** Structure describing GPU culling setup for a single Proxy. */
	struct FProxyDesc
	{
		FRHITexture* PageTableTexture;
		FRHITexture* HeightMinMaxTexture;
		FRHITexture* LodBiasMinMaxTexture;
		int32 MinMaxLevelOffset;

		uint32 MaxLevel;
		uint32 NumForceLoadLods;
		uint32 PageTableFeedbackId;
		uint32 NumPhysicalAddressBits;
		FVector4 PageTableSize;
		FVector4 PhysicalPageTransform;
		FMatrix UVToWorld;
		FVector UVToWorldScale;
		uint32 NumQuadsPerTileSide;

		int32 MaxPersistentQueueItems;
		int32 MaxRenderItems;
		int32 MaxFeedbackItems;
		int32 NumCollectPassWavefronts;
	};

	/** View description used for LOD calculation in the main view. */
	struct FMainViewDesc
	{
		FSceneView const* ViewDebug;
		FVector ViewOrigin;
		FVector4 LodDistances;
		float LodBiasScale;
		FVector4 Planes[5];
		FTextureRHIRef OcclusionTexture;
		int32 OcclusionLevelOffset;
	};

	/** View description used for culling in the child view. */
	struct FChildViewDesc
	{
		FSceneView const* ViewDebug;
		bool bIsMainView;
		FVector4 Planes[5];
	};

	/** Structure to carry RDG resources. */
	struct FVolatileResources
	{
		FRDGBufferRef QueueInfo;
		FRDGBufferUAVRef QueueInfoUAV;
		FRDGBufferRef QueueBuffer;
		FRDGBufferUAVRef QueueBufferUAV;

		FRDGBufferRef QuadBuffer;
		FRDGBufferUAVRef QuadBufferUAV;
		FRDGBufferSRVRef QuadBufferSRV;

		FRDGBufferRef FeedbackBuffer;
		FRDGBufferUAVRef FeedbackBufferUAV;

		FRDGBufferRef IndirectArgsBuffer;
		FRDGBufferUAVRef IndirectArgsBufferUAV;
		FRDGBufferSRVRef IndirectArgsBufferSRV;
	};

	/** Initialize the FDrawInstanceBuffers objects. */
	void InitializeInstanceBuffers(FRHICommandListBase& RHICmdList, FDrawInstanceBuffers& InBuffers)
	{
		{
			const FRHIBufferCreateDesc CreateDesc =
				FRHIBufferCreateDesc::CreateStructured<VirtualHeightfieldMesh::QuadRenderInstance>(TEXT("VirtualHeightfieldMesh.InstanceBuffer"), CVarVHMMaxRenderItems.GetValueOnRenderThread())
				.AddUsage(EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::ShaderResource)
				.SetInitialState(ERHIAccess::SRVMask);

			InBuffers.InstanceBuffer = RHICmdList.CreateBuffer(CreateDesc);
			InBuffers.InstanceBufferUAV = RHICmdList.CreateUnorderedAccessView(
				InBuffers.InstanceBuffer, 
				FRHIViewDesc::CreateBufferUAV()
					.SetTypeFromBuffer(InBuffers.InstanceBuffer));
			InBuffers.InstanceBufferSRV = RHICmdList.CreateShaderResourceView(InBuffers.InstanceBuffer, FRHIViewDesc::CreateBufferSRV().SetTypeFromBuffer(InBuffers.InstanceBuffer));
		}
		{
			const FRHIBufferCreateDesc CreateDesc =
				FRHIBufferCreateDesc::CreateVertex<uint32>(TEXT("VirtualHeightfieldMesh.InstanceIndirectArgsBuffer"), 5)
				.AddUsage(EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::DrawIndirect)
				.SetInitialState(ERHIAccess::IndirectArgs);

			InBuffers.IndirectArgsBuffer = RHICmdList.CreateBuffer(CreateDesc);
			InBuffers.IndirectArgsBufferUAV = RHICmdList.CreateUnorderedAccessView(
				InBuffers.IndirectArgsBuffer, 
				FRHIViewDesc::CreateBufferUAV()
					.SetType(FRHIViewDesc::EBufferType::Typed)
					.SetFormat(PF_R32_UINT));
		}
	}

	/** Initialize the volatile resources used in the render graph. */
	void InitializeResources(FRDGBuilder& GraphBuilder, FProxyDesc const& InDesc, FMainViewDesc const& InMainViewDesc, FVolatileResources& OutResources)
	{
		OutResources.QueueInfo = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(WorkerQueueInfo), 1), TEXT("VirtualHeightfieldMesh.QueueInfo"));
		OutResources.QueueInfoUAV = GraphBuilder.CreateUAV(OutResources.QueueInfo);
		OutResources.QueueBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), InDesc.MaxPersistentQueueItems), TEXT("VirtualHeightfieldMesh.QuadQueue"));
		OutResources.QueueBufferUAV = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(OutResources.QueueBuffer, PF_R32_UINT));

		OutResources.QuadBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32) * 2, InDesc.MaxRenderItems), TEXT("VirtualHeightfieldMesh.QuadBuffer"));
		OutResources.QuadBufferUAV = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(OutResources.QuadBuffer, PF_R32G32_UINT));
		OutResources.QuadBufferSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(OutResources.QuadBuffer, PF_R32G32_UINT));

		FRDGBufferDesc FeedbackBufferDesc = FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), InDesc.MaxFeedbackItems);
		FeedbackBufferDesc.Usage = EBufferUsageFlags(FeedbackBufferDesc.Usage | BUF_SourceCopy);
		OutResources.FeedbackBuffer = GraphBuilder.CreateBuffer(FeedbackBufferDesc, TEXT("VirtualHeightfieldMesh.FeedbackBuffer"));
		OutResources.FeedbackBufferUAV = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(OutResources.FeedbackBuffer, PF_R32_UINT));

		OutResources.IndirectArgsBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc(IndirectArgsByteSize), TEXT("VirtualHeightfieldMesh.IndirectArgsBuffer"));
		OutResources.IndirectArgsBufferUAV = GraphBuilder.CreateUAV(OutResources.IndirectArgsBuffer);
		OutResources.IndirectArgsBufferSRV = GraphBuilder.CreateSRV(OutResources.IndirectArgsBuffer);
	}

	/** Transition our output draw buffers for use. Read or write access is set according to the bToWrite parameter. */
	void AddPass_TransitionAllDrawBuffers(FRDGBuilder& GraphBuilder, TArray<VirtualHeightfieldMesh::FDrawInstanceBuffers> const& Buffers, TArrayView<int32> const& BufferIndices, bool bToWrite)
	{
		TArray<FRHIUnorderedAccessView*> OverlapUAVs;
		OverlapUAVs.Reserve(BufferIndices.Num());

		TArray<FRHITransitionInfo> TransitionInfos;
		TransitionInfos.Reserve(BufferIndices.Num() * 2);
		
		for (int32 BufferIndex : BufferIndices)
		{
			FRHIUnorderedAccessView* IndirectArgsBufferUAV = Buffers[BufferIndex].IndirectArgsBufferUAV;
			FRHIUnorderedAccessView* InstanceBufferUAV = Buffers[BufferIndex].InstanceBufferUAV;

			OverlapUAVs.Add(IndirectArgsBufferUAV);

			TransitionInfos.Add(FRHITransitionInfo(IndirectArgsBufferUAV, bToWrite ? ERHIAccess::IndirectArgs : ERHIAccess::UAVMask, bToWrite ? ERHIAccess::UAVMask : ERHIAccess::IndirectArgs));
			TransitionInfos.Add(FRHITransitionInfo(InstanceBufferUAV, bToWrite ? ERHIAccess::SRVMask : ERHIAccess::UAVMask, bToWrite ? ERHIAccess::UAVMask : ERHIAccess::SRVMask));
		}

		AddPass(GraphBuilder, RDG_EVENT_NAME("TransitionAllDrawBuffers"), [bToWrite, OverlapUAVs, TransitionInfos](FRHICommandList& InRHICmdList)
		{
			if (!bToWrite)
			{
				InRHICmdList.EndUAVOverlap(OverlapUAVs);
			}

			InRHICmdList.Transition(TransitionInfos);
			
			if (bToWrite)
			{
				InRHICmdList.BeginUAVOverlap(OverlapUAVs);
			}
		});
	}

	/** Initialize the buffers before collecting visible quads. */
	void AddPass_InitBuffers(FRDGBuilder& GraphBuilder, FGlobalShaderMap* InGlobalShaderMap, FProxyDesc const& InDesc, FVolatileResources& InVolatileResources)
	{
		TShaderMapRef<FInitBuffersVHM_CS> ComputeShader(InGlobalShaderMap);

		FInitBuffersVHM_CS::FParameters* PassParameters = GraphBuilder.AllocParameters<FInitBuffersVHM_CS::FParameters>();
		PassParameters->MaxLevel = InDesc.MaxLevel;
		PassParameters->NumForceLoadLods = InDesc.NumForceLoadLods;
		PassParameters->PageTableFeedbackId = InDesc.PageTableFeedbackId;
		PassParameters->RWQueueInfo = InVolatileResources.QueueInfoUAV;
		PassParameters->RWQueueBuffer = InVolatileResources.QueueBufferUAV;
		PassParameters->RWQuadBuffer = InVolatileResources.QuadBufferUAV;
		PassParameters->RWIndirectArgsBuffer = InVolatileResources.IndirectArgsBufferUAV;
		PassParameters->RWFeedbackBuffer = InVolatileResources.FeedbackBufferUAV;

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("InitBuffers"),
			PassParameters,
			ERDGPassFlags::Compute,
			[PassParameters, ComputeShader](FRHICommandList& RHICmdList)
		{
			//todo: If feedback parsing understands append counter we don't need to fully clear
			RHICmdList.ClearUAVUint(PassParameters->RWFeedbackBuffer->GetRHI(), FUintVector4(0xffffffff, 0xffffffff, 0xffffffff, 0xffffffff));

			FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, *PassParameters, FIntVector(1, 1, 1));
		});
	}

	/** Collect potentially visible quads and determine their Lods. */
	void AddPass_CollectQuads(FRDGBuilder& GraphBuilder, FGlobalShaderMap* InGlobalShaderMap, FProxyDesc const& InDesc, FVolatileResources& InVolatileResources, FMainViewDesc const& InViewDesc)
	{
		TShaderMapRef<FCollectQuadsVHM_CS> ComputeShader(InGlobalShaderMap);

		FCollectQuadsVHM_CS::FParameters* PassParameters = GraphBuilder.AllocParameters<FCollectQuadsVHM_CS::FParameters>();
		PassParameters->HeightMinMaxTexture = InDesc.HeightMinMaxTexture;
		PassParameters->LodBiasMinMaxTexture = InDesc.LodBiasMinMaxTexture;
		PassParameters->MinMaxTextureSampler = TStaticSamplerState<SF_Point>::GetRHI();
		PassParameters->MinMaxLevelOffset = InDesc.MinMaxLevelOffset;
		PassParameters->OcclusionTexture = InViewDesc.OcclusionTexture;
		PassParameters->OcclusionLevelOffset = InViewDesc.OcclusionLevelOffset;
		PassParameters->PageTableTexture = InDesc.PageTableTexture;
		PassParameters->MaxLevel = InDesc.MaxLevel;
		PassParameters->PageTableSize = FVector4f(InDesc.PageTableSize); // LWC_TODO: precision loss
		PassParameters->PageTableFeedbackId = InDesc.PageTableFeedbackId;
		PassParameters->UVToWorld = FMatrix44f(InDesc.UVToWorld);		// LWC_TODO: Precision loss
		PassParameters->UVToWorldScale = (FVector3f)InDesc.UVToWorldScale;
		PassParameters->ViewOrigin = (FVector3f)InViewDesc.ViewOrigin;
		PassParameters->LodDistances = FVector4f(InViewDesc.LodDistances); // LWC_TODO: precision loss
		PassParameters->LodBiasScale = InViewDesc.LodBiasScale;
		for (int32 PlaneIndex = 0; PlaneIndex < 5; ++PlaneIndex)
		{
			PassParameters->FrustumPlanes[PlaneIndex] = FVector4f(InViewDesc.Planes[PlaneIndex]); // LWC_TODO: precision loss
		}
		PassParameters->QueueBufferSizeMask = InDesc.MaxPersistentQueueItems - 1; // Assumes MaxPersistentQueueItems is a power of 2 so that we can wrap with a mask.
		PassParameters->RWQueueInfo = InVolatileResources.QueueInfoUAV;
		PassParameters->RWQueueBuffer = InVolatileResources.QueueBufferUAV;
		PassParameters->RWQuadBuffer = InVolatileResources.QuadBufferUAV;
		PassParameters->RWIndirectArgsBuffer = InVolatileResources.IndirectArgsBufferUAV;
		PassParameters->RWFeedbackBuffer = InVolatileResources.FeedbackBufferUAV;

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("CollectQuads"),
			ComputeShader, PassParameters, FIntVector(InDesc.NumCollectPassWavefronts, 1, 1));
	}

	/** Initialise the draw indirect buffer. */
	void AddPass_InitInstanceBuffer(FRDGBuilder& GraphBuilder, FGlobalShaderMap* InGlobalShaderMap, int32 NumQuadsPerTileSide, FDrawInstanceBuffers& InOutputResources)
	{
		TShaderMapRef<FInitInstanceBufferVHM_CS> ComputeShader(InGlobalShaderMap);

		FInitInstanceBufferVHM_CS::FParameters* PassParameters = GraphBuilder.AllocParameters<FInitInstanceBufferVHM_CS::FParameters>();
		PassParameters->NumIndices = NumQuadsPerTileSide * NumQuadsPerTileSide * 6;
		PassParameters->RWIndirectArgsBuffer = InOutputResources.IndirectArgsBufferUAV;

 		FComputeShaderUtils::AddPass(
 			GraphBuilder,
 			RDG_EVENT_NAME("InitInstanceBuffer"),
 			ComputeShader, PassParameters, FIntVector(1, 1, 1));
	}

	/** Cull quads and write to the final output buffer. */
	void AddPass_CullInstances(FRDGBuilder& GraphBuilder, FGlobalShaderMap* InGlobalShaderMap, FProxyDesc const& InDesc, FVolatileResources& InVolatileResources, FDrawInstanceBuffers& InOutputResources, FChildViewDesc const& InViewDesc)
	{
		FCullInstancesVHM_CS::FParameters* PassParameters = GraphBuilder.AllocParameters<FCullInstancesVHM_CS::FParameters>();
		PassParameters->HeightMinMaxTexture = InDesc.HeightMinMaxTexture;
		PassParameters->MinMaxTextureSampler = TStaticSamplerState<SF_Point>::GetRHI();
		PassParameters->MinMaxLevelOffset = InDesc.MinMaxLevelOffset;
		PassParameters->PageTableTexture = InDesc.PageTableTexture;
		PassParameters->PageTableSize = FVector4f(InDesc.PageTableSize); // LWC_TODO: precision loss
		for (int32 PlaneIndex = 0; PlaneIndex < 5; ++PlaneIndex)
		{
			PassParameters->FrustumPlanes[PlaneIndex] = FVector4f(InViewDesc.Planes[PlaneIndex]); // LWC_TODO: precision loss
		}
		PassParameters->PhysicalPageTransform = FVector4f(InDesc.PhysicalPageTransform); // LWC_TODO: precision loss
		PassParameters->NumPhysicalAddressBits = InDesc.NumPhysicalAddressBits;
		PassParameters->QuadBuffer = InVolatileResources.QuadBufferSRV;
		PassParameters->IndirectArgsBuffer = InVolatileResources.IndirectArgsBuffer;
		PassParameters->IndirectArgsBufferSRV = InVolatileResources.IndirectArgsBufferSRV;
		PassParameters->RWInstanceBuffer = InOutputResources.InstanceBufferUAV;
		PassParameters->RWIndirectArgsBuffer = InOutputResources.IndirectArgsBufferUAV;

		int32 IndirectArgOffset = VirtualHeightfieldMesh::IndirectArgsByteOffset_FinalCull;

		FCullInstancesVHM_CS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FCullInstancesVHM_CS::FReuseCullDim>(InViewDesc.bIsMainView);

		TShaderMapRef<FCullInstancesVHM_CS> ComputeShader(InGlobalShaderMap, PermutationVector);
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("CullInstances"),
			ComputeShader, PassParameters,
			InVolatileResources.IndirectArgsBuffer,
			IndirectArgOffset);
	}
}

void FVirtualHeightfieldMeshRendererExtension::SubmitWork(FRDGBuilder& GraphBuilder)
{
	SCOPE_CYCLE_COUNTER(STAT_VirtualHeightfieldMesh_SubmitWork);
	RDG_EVENT_SCOPE_STAT(GraphBuilder, VirtualHeightfieldMesh, "VirtualHeightfieldMesh");

	// Sort work so that we can batch by proxy/view
	WorkDescs.Sort(FWorkDescSort());

	// Add pass to transition all output buffers for writing
	TArray<int32, TInlineAllocator<8>> UsedBufferIndices;
	for (FWorkDesc WorkdDesc : WorkDescs)
	{
		UsedBufferIndices.Add(WorkdDesc.BufferIndex);
	}
	AddPass_TransitionAllDrawBuffers(GraphBuilder, Buffers, UsedBufferIndices, true);

	// Add passes to initialize the output buffers
	for (FWorkDesc WorkDesc : WorkDescs)
	{
		const int32 NumQuadsPerTileSide = SceneProxies[WorkDesc.ProxyIndex]->NumQuadsPerTileSide;
		AddPass_InitInstanceBuffer(GraphBuilder, GetGlobalShaderMap(GMaxRHIFeatureLevel), NumQuadsPerTileSide, Buffers[WorkDesc.BufferIndex]);
	}

	// Iterate workloads and submit work
	const int32 NumWorkItems = WorkDescs.Num();
	int32 WorkIndex = 0;
	while (WorkIndex < NumWorkItems)
	{
		// Gather data per proxy
		FVirtualHeightfieldMeshSceneProxy const* Proxy = SceneProxies[WorkDescs[WorkIndex].ProxyIndex];
		IAllocatedVirtualTexture* AllocatedVirtualTexture = Proxy->AllocatedVirtualTexture;
			
		const float PageSize = AllocatedVirtualTexture->GetVirtualTileSize();
		const float PageBorderSize = AllocatedVirtualTexture->GetTileBorderSize();
		const float PageAndBorderSize = PageSize + PageBorderSize * 2.f;
		const float HalfTexelSize = 0.5f;
		const float PhysicalTextureSize = AllocatedVirtualTexture->GetPhysicalTextureSize(0);
		const FVector4 PhysicalPageTransform = FVector4(PageAndBorderSize, PageSize, PageBorderSize, HalfTexelSize) * (1.f / PhysicalTextureSize);

		const float PageTableSizeX = AllocatedVirtualTexture->GetWidthInTiles();
		const float PageTableSizeY = AllocatedVirtualTexture->GetHeightInTiles();
		const FVector4 PageTableSize = FVector4(PageTableSizeX, PageTableSizeY, 1.f / PageTableSizeX, 1.f / PageTableSizeY);

		VirtualHeightfieldMesh::FProxyDesc ProxyDesc;
		ProxyDesc.PageTableTexture = AllocatedVirtualTexture->GetPageTableTexture(0);
		ProxyDesc.HeightMinMaxTexture = Proxy->HeightMinMaxTexture ? Proxy->HeightMinMaxTexture->GetResource()->TextureRHI : VirtualHeightfieldMesh::GHeightMinMaxDefaultTexture->TextureRHI;
		ProxyDesc.LodBiasMinMaxTexture = Proxy->LodBiasMinMaxTexture ? Proxy->LodBiasMinMaxTexture->GetResource()->TextureRHI : GBlackTexture->TextureRHI;
		ProxyDesc.MinMaxLevelOffset = ProxyDesc.HeightMinMaxTexture->GetNumMips() - 1 - AllocatedVirtualTexture->GetMaxLevel();
		ProxyDesc.MaxLevel = AllocatedVirtualTexture->GetMaxLevel();
		ProxyDesc.NumForceLoadLods = FMath::Min<uint32>(FMath::Max(Proxy->NumForceLoadLods, 0), ProxyDesc.MaxLevel + 1);
		ProxyDesc.PageTableSize = PageTableSize;
		ProxyDesc.PhysicalPageTransform = PhysicalPageTransform;
		ProxyDesc.NumPhysicalAddressBits = AllocatedVirtualTexture->GetPageTableFormat() == EVTPageTableFormat::UInt16 ? 6 : 8; // See packing in PageTableUpdate.usf
		ProxyDesc.PageTableFeedbackId = AllocatedVirtualTexture->GetSpaceID() << 28;
		ProxyDesc.UVToWorld = Proxy->UVToWorld;
		ProxyDesc.UVToWorldScale = Proxy->UVToWorldScale;
		ProxyDesc.NumQuadsPerTileSide = Proxy->NumQuadsPerTileSide;
		ProxyDesc.MaxPersistentQueueItems = 1 << FMath::CeilLogTwo(CVarVHMMaxPersistentQueueItems.GetValueOnRenderThread());
		ProxyDesc.MaxRenderItems = CVarVHMMaxRenderItems.GetValueOnRenderThread();
		ProxyDesc.MaxFeedbackItems = CVarVHMMaxFeedbackItems.GetValueOnRenderThread();
		ProxyDesc.NumCollectPassWavefronts = CVarVHMCollectPassWavefronts.GetValueOnRenderThread();

		while (WorkIndex < NumWorkItems && SceneProxies[WorkDescs[WorkIndex].ProxyIndex] == Proxy)
		{
			// Gather data per main view
			FSceneView const* MainView = MainViews[WorkDescs[WorkIndex].MainViewIndex];
				
			VirtualHeightfieldMesh::FViewData MainViewData;
			VirtualHeightfieldMesh::GetViewData(MainView, MainViewData);

			VirtualHeightfieldMesh::FMainViewDesc MainViewDesc;
			MainViewDesc.ViewDebug = MainView;

			// ViewOrigin and Frustum Planes are all converted to UV space for the shader.
			MainViewDesc.ViewOrigin = Proxy->WorldToUV.TransformPosition(MainViewData.ViewOrigin);
			MainViewDesc.LodDistances = FVector4(VirtualHeightfieldMesh::CalculateLodRanges(MainView, Proxy));
			MainViewDesc.LodBiasScale = Proxy->LodBiasScale;

			const int32 MainViewNumPlanes = FMath::Min(MainViewData.ViewFrustum.Planes.Num(), 5);
			for (int32 PlaneIndex = 0; PlaneIndex < MainViewNumPlanes; ++PlaneIndex)
			{
				FPlane Plane = MainViewData.ViewFrustum.Planes[PlaneIndex];
				Plane = VirtualHeightfieldMesh::TransformPlane(Plane, Proxy->WorldToUV, Proxy->WorldToUVTransposeAdjoint);
				MainViewDesc.Planes[PlaneIndex] = VirtualHeightfieldMesh::ConvertPlane(Plane);
			}
			for (int32 PlaneIndex = MainViewNumPlanes; PlaneIndex < 5; ++PlaneIndex)
			{
				MainViewDesc.Planes[PlaneIndex] = FPlane(0, 0, 0, 1); // Null plane won't cull anything
			}

			FOcclusionResults* OcclusionResults = GOcclusionResults.Find(FOcclusionResultsKey(Proxy, MainView));
			if (OcclusionResults == nullptr)
			{
				MainViewDesc.OcclusionTexture = GBlackTexture->TextureRHI;
				MainViewDesc.OcclusionLevelOffset = (int32)ProxyDesc.MaxLevel;
			}
			else
			{
				if (!OcclusionResults->UploadData.IsEmpty())
				{
					const FRHITextureCreateDesc Desc =
						FRHITextureCreateDesc::Create2D(TEXT("VirtualHeightfieldMesh.OcclusionTexture"), OcclusionResults->TextureSize, PF_G8)
						.SetNumMips(OcclusionResults->NumTextureMips)
						.SetFlags(ETextureCreateFlags::ShaderResource);

					OcclusionResults->OcclusionTexture = GraphBuilder.RHICmdList.CreateTexture(Desc);

					int32 SourceIndex = 0;

					FIntPoint Size = OcclusionResults->TextureSize;
					for (int32 MipIndex = 0; MipIndex < OcclusionResults->NumTextureMips; ++MipIndex)
					{
						uint32 Stride;
						uint8* Dst = (uint8*)GraphBuilder.RHICmdList.LockTexture2D(OcclusionResults->OcclusionTexture, MipIndex, RLM_WriteOnly, Stride, false);

						for (int Y = 0; Y < Size.Y; ++Y)
						{
							for (int X = 0; X < Size.X; ++X)
							{
								Dst[Y * Stride + X] = OcclusionResults->UploadData[SourceIndex++] ? 255 : 0;
							}
						}

						GraphBuilder.RHICmdList.UnlockTexture2D(OcclusionResults->OcclusionTexture, MipIndex, false);

						Size.X = FMath::Max(Size.X / 2, 1);
						Size.Y = FMath::Max(Size.Y / 2, 1);
					}

					OcclusionResults->UploadData.Empty();
				}

				MainViewDesc.OcclusionTexture = OcclusionResults->OcclusionTexture;
				MainViewDesc.OcclusionLevelOffset = (int32)ProxyDesc.MaxLevel - OcclusionResults->NumTextureMips + 1;
			}

			// Build volatile graph resources
			VirtualHeightfieldMesh::FVolatileResources VolatileResources;
			VirtualHeightfieldMesh::InitializeResources(GraphBuilder, ProxyDesc, MainViewDesc, VolatileResources);

			// Build graph
			VirtualHeightfieldMesh::AddPass_InitBuffers(GraphBuilder, GetGlobalShaderMap(GMaxRHIFeatureLevel), ProxyDesc, VolatileResources);
			VirtualHeightfieldMesh::AddPass_CollectQuads(GraphBuilder, GetGlobalShaderMap(GMaxRHIFeatureLevel), ProxyDesc, VolatileResources, MainViewDesc);

			FVirtualTextureFeedbackBufferDesc Desc;
 			Desc.BufferSize = CVarVHMMaxFeedbackItems.GetValueOnRenderThread();
			Desc.bSizeInHeader = true;
 			SubmitVirtualTextureFeedbackBuffer(GraphBuilder, VolatileResources.FeedbackBuffer, Desc);

			while (WorkIndex < NumWorkItems && SceneProxies[WorkDescs[WorkIndex].ProxyIndex] == Proxy && MainViews[WorkDescs[WorkIndex].MainViewIndex] == MainView)
			{
				// Gather data per child view
				FSceneView const* CullView = CullViews[WorkDescs[WorkIndex].CullViewIndex];
				FConvexVolume const* ShadowFrustum = CullView->GetDynamicMeshElementsShadowCullFrustum();
				FConvexVolume const& Frustum = ShadowFrustum != nullptr && ShadowFrustum->Planes.Num() > 0 ? *ShadowFrustum : CullView->ViewFrustum;
				const FVector PreShadowTranslation = ShadowFrustum != nullptr ? CullView->GetPreShadowTranslation() : FVector::ZeroVector;

				VirtualHeightfieldMesh::FChildViewDesc ChildViewDesc;
				ChildViewDesc.ViewDebug = MainView;
				ChildViewDesc.bIsMainView = CullView == MainView;
					
				const int32 ChildViewNumPlanes = FMath::Min(Frustum.Planes.Num(), 5);
				for (int32 PlaneIndex = 0; PlaneIndex < ChildViewNumPlanes; ++PlaneIndex)
				{
					FPlane Plane = Frustum.Planes[PlaneIndex];
					Plane = VirtualHeightfieldMesh::TranslatePlane(Plane, PreShadowTranslation);
					Plane = VirtualHeightfieldMesh::TransformPlane(Plane, Proxy->WorldToUV, Proxy->WorldToUVTransposeAdjoint);
					ChildViewDesc.Planes[PlaneIndex] = VirtualHeightfieldMesh::ConvertPlane(Plane);
				}
				for (int32 PlaneIndex = ChildViewNumPlanes; PlaneIndex < 5; ++PlaneIndex)
				{
					MainViewDesc.Planes[PlaneIndex] = FPlane(0, 0, 0, 1); // Null plane won't cull anything
				}

				// Build graph
				VirtualHeightfieldMesh::AddPass_CullInstances(GraphBuilder, GetGlobalShaderMap(GMaxRHIFeatureLevel), ProxyDesc, VolatileResources, Buffers[WorkDescs[WorkIndex].BufferIndex], ChildViewDesc);

				WorkIndex++;
			}
		}
	}

	// Add pass to transition all output buffers for reading
	AddPass_TransitionAllDrawBuffers(GraphBuilder, Buffers, UsedBufferIndices, false);
}
