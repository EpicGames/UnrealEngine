// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Materials/MaterialRelevance.h"
#include "PrimitiveSceneProxy.h"

class FMeshElementCollector;
union FVirtualTextureProducerHandle;

/** Scene proxy responsible for rendering of a Virtual Heightfield Mesh. */
class FVirtualHeightfieldMeshSceneProxy final : public FPrimitiveSceneProxy
{
public:
	FVirtualHeightfieldMeshSceneProxy(class UVirtualHeightfieldMeshComponent* InComponent);

protected:
	//~ Begin FPrimitiveSceneProxy Interface
	virtual SIZE_T GetTypeHash() const override;
	virtual uint32 GetMemoryFootprint() const override;
	virtual void CreateRenderThreadResources(FRHICommandListBase& RHICmdList) override;
	virtual void DestroyRenderThreadResources() override;
	virtual void OnTransformChanged(FRHICommandListBase& RHICmdList) override;
	virtual bool HasSubprimitiveOcclusionQueries() const override;
	virtual const TArray<FBoxSphereBounds>* GetOcclusionQueries(const FSceneView* View) const override;
	virtual void AcceptOcclusionResults(const FSceneView* View, TArray<bool>* Results, int32 ResultsStart, int32 NumResults) override;
	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override;
	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override;
	//~ End FPrimitiveSceneProxy Interface

private:
	void BuildOcclusionVolumes(TArrayView<FVector2D> const& InMinMaxData, FIntPoint const& InMinMaxSize, TArrayView<int32> const& InMinMaxMips, int32 InNumLods);
	static void OnVirtualTextureDestroyedCB(const FVirtualTextureProducerHandle& InHandle, void* Baton);

public:
	bool bHiddenInEditor;

	class URuntimeVirtualTexture* RuntimeVirtualTexture;
	class FMaterialRenderProxy* Material;
	FMaterialRelevance MaterialRelevance;

	class UTexture2D* HeightMinMaxTexture;
	class UTexture2D* LodBiasTexture;
	class UTexture2D* LodBiasMinMaxTexture;

	class IAllocatedVirtualTexture* AllocatedVirtualTexture;
	bool bCallbackRegistered;

	uint32 NumQuadsPerTileSide;
	
	FVector UVToWorldScale;
	FMatrix UVToLocal;
	FMatrix UVToWorld;
	FMatrix WorldToUV;
	FMatrix WorldToUVTransposeAdjoint;

	class FVirtualHeightfieldMeshVertexFactory* VertexFactory;

	float Lod0ScreenSize;
	float Lod0Distribution;
	float LodDistribution;
	float LodBiasScale;
	
	int32 NumForceLoadLods;

	TArray<FVector2D> OcclusionData;
	int32 NumOcclusionLods;
	FIntPoint OcclusionGridSize;
	TArray<FBoxSphereBounds> OcclusionVolumes;
	TArray<FBoxSphereBounds> DefaultOcclusionVolumes;
};
