// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualHeightfieldMeshVertexFactory.h"

#include "MaterialDomain.h"
#include "MeshDrawShaderBindings.h"
#include "MeshMaterialShader.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RHIResourceUtils.h"

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FVirtualHeightfieldMeshVertexFactoryParameters, "VHM");

namespace
{
	template< typename T >
	FBufferRHIRef CreateIndexBuffer(FRHICommandListBase& RHICmdList, uint32 NumQuadsPerSide)
	{
		TArray<T> Indices;

		// Allocate room for indices
		Indices.Reserve(NumQuadsPerSide * NumQuadsPerSide * 6);

		// Build index buffer in morton order for better vertex reuse. This amounts to roughly 75% reuse rate vs 66% of naive scanline approach
		for (uint32 Morton = 0; Morton < NumQuadsPerSide * NumQuadsPerSide; Morton++)
		{
			uint32 SquareX = FMath::ReverseMortonCode2(Morton);
			uint32 SquareY = FMath::ReverseMortonCode2(Morton >> 1);

			bool ForwardDiagonal = false;

#if 0
			// todo: Support odd/even topology for all landscape stuff
 			if (SquareX % 2)
 			{
 				ForwardDiagonal = !ForwardDiagonal;
 			}
 			if (SquareY % 2)
 			{
 				ForwardDiagonal = !ForwardDiagonal;
 			}
#endif
			int32 Index0 = SquareX + SquareY * (NumQuadsPerSide + 1);
			int32 Index1 = Index0 + 1;
			int32 Index2 = Index0 + (NumQuadsPerSide + 1);
			int32 Index3 = Index2 + 1;

			if (!ForwardDiagonal)
			{
				Indices.Add(Index3);
				Indices.Add(Index1);
				Indices.Add(Index0);
				Indices.Add(Index0);
				Indices.Add(Index2);
				Indices.Add(Index3);
			}
			else
			{
				Indices.Add(Index3);
				Indices.Add(Index1);
				Indices.Add(Index2);
				Indices.Add(Index0);
				Indices.Add(Index2);
				Indices.Add(Index1);
			}
		}

		return UE::RHIResourceUtils::CreateIndexBufferFromArray(RHICmdList, TEXT("FVirtualHeightfieldMeshIndexBuffer"), EBufferUsageFlags::Static, MakeConstArrayView(Indices));
	}
}

void FVirtualHeightfieldMeshIndexBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	NumIndices = NumQuadsPerSide * NumQuadsPerSide * 6;
	if (NumQuadsPerSide < 256)
	{
		IndexBufferRHI = CreateIndexBuffer<uint16>(RHICmdList, NumQuadsPerSide);
	}
	else
	{
		IndexBufferRHI = CreateIndexBuffer<uint32>(RHICmdList, NumQuadsPerSide);
	}
}


/**
 * Shader parameters for vertex factory.
 */
class FVirtualHeightfieldMeshVertexFactoryShaderParameters : public FVertexFactoryShaderParameters
{
	DECLARE_TYPE_LAYOUT(FVirtualHeightfieldMeshVertexFactoryShaderParameters, NonVirtual);

public:
	void Bind(const FShaderParameterMap& ParameterMap)
	{
		InstanceBufferParameter.Bind(ParameterMap, TEXT("InstanceBuffer"));
		LodViewOriginParameter.Bind(ParameterMap, TEXT("LodViewOrigin"));
		LodDistancesParameter.Bind(ParameterMap, TEXT("LodDistances"));
	}

	void GetElementShaderBindings(
		const class FSceneInterface* Scene,
		const class FSceneView* View,
		const class FMeshMaterialShader* Shader,
		const EVertexInputStreamType InputStreamType,
		ERHIFeatureLevel::Type FeatureLevel,
		const class FVertexFactory* InVertexFactory,
		const struct FMeshBatchElement& BatchElement,
		class FMeshDrawSingleShaderBindings& ShaderBindings,
		FVertexInputStreamArray& VertexStreams) const
	{
		FVirtualHeightfieldMeshVertexFactory* VertexFactory = (FVirtualHeightfieldMeshVertexFactory*)InVertexFactory;
		ShaderBindings.Add(Shader->GetUniformBufferParameter<FVirtualHeightfieldMeshVertexFactoryParameters>(), VertexFactory->UniformBuffer);

		FVirtualHeightfieldMeshUserData* UserData = (FVirtualHeightfieldMeshUserData*)BatchElement.UserData;
		ShaderBindings.Add(InstanceBufferParameter, UserData->InstanceBufferSRV);
		ShaderBindings.Add(LodViewOriginParameter, UserData->LodViewOrigin);
		ShaderBindings.Add(LodDistancesParameter, UserData->LodDistances);
	}

protected:
	LAYOUT_FIELD(FShaderResourceParameter, InstanceBufferParameter);
	LAYOUT_FIELD(FShaderParameter, LodViewOriginParameter);
	LAYOUT_FIELD(FShaderParameter, LodDistancesParameter);
};

IMPLEMENT_TYPE_LAYOUT(FVirtualHeightfieldMeshVertexFactoryShaderParameters);

IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FVirtualHeightfieldMeshVertexFactory, SF_Vertex, FVirtualHeightfieldMeshVertexFactoryShaderParameters);
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FVirtualHeightfieldMeshVertexFactory, SF_Pixel, FVirtualHeightfieldMeshVertexFactoryShaderParameters);


FVirtualHeightfieldMeshVertexFactory::FVirtualHeightfieldMeshVertexFactory(ERHIFeatureLevel::Type InFeatureLevel, const FVirtualHeightfieldMeshVertexFactoryParameters& InParams)
	: FVertexFactory(InFeatureLevel)
	, Params(InParams)
{
	IndexBuffer = new FVirtualHeightfieldMeshIndexBuffer(InParams.NumQuadsPerTileSide);
}

FVirtualHeightfieldMeshVertexFactory::~FVirtualHeightfieldMeshVertexFactory()
{
	delete IndexBuffer;
}

void FVirtualHeightfieldMeshVertexFactory::InitRHI(FRHICommandListBase& RHICmdList)
{
	UniformBuffer = FVirtualHeightfieldMeshVertexFactoryBufferRef::CreateUniformBufferImmediate(Params, UniformBuffer_MultiFrame);

	IndexBuffer->InitResource(RHICmdList);

	FVertexStream NullVertexStream;
	NullVertexStream.VertexBuffer = nullptr;
	NullVertexStream.Stride = 0;
	NullVertexStream.Offset = 0;
	NullVertexStream.VertexStreamUsage = EVertexStreamUsage::ManualFetch;

	check(Streams.Num() == 0);
	Streams.Add(NullVertexStream);

	FVertexDeclarationElementList Elements;
	//Elements.Add(FVertexElement(0, 0, VET_None, 0, 0, false));

#if 0
	const bool bCanUseGPUScene = UseGPUScene(GMaxRHIShaderPlatform, GMaxRHIFeatureLevel);

	// Add support for primitiveID
	const uint8 Index = static_cast<uint8>(EVertexInputStreamType::Default);
	PrimitiveIdStreamIndex[Index] = -1;
	if (GetType()->SupportsPrimitiveIdStream() && bCanUseGPUScene)
	{
		// When the VF is used for rendering in normal mesh passes, this vertex buffer and offset will be overridden
		Elements.Add(AccessStreamComponent(FVertexStreamComponent(&GPrimitiveIdDummy, 0, 0, sizeof(uint32), VET_UInt, EVertexStreamUsage::Instancing), 13));
		PrimitiveIdStreamIndex[Index] = Elements.Last().StreamIndex;
	}
#endif

	InitDeclaration(Elements);
}

void FVirtualHeightfieldMeshVertexFactory::ReleaseRHI()
{
	UniformBuffer.SafeRelease();

	if (IndexBuffer)
	{
		IndexBuffer->ReleaseResource();
	}

	FVertexFactory::ReleaseRHI();
}

bool FVirtualHeightfieldMeshVertexFactory::ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters)
{
	//todo[vhm]: Fallback path for mobile.
	if (!IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5))
	{
		return false;
	}
	return (Parameters.MaterialParameters.MaterialDomain == MD_Surface && Parameters.MaterialParameters.bIsUsedWithVirtualHeightfieldMesh) || Parameters.MaterialParameters.bIsSpecialEngineMaterial;
}

void FVirtualHeightfieldMeshVertexFactory::ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	OutEnvironment.SetDefine(TEXT("VF_VIRTUAL_HEIGHFIELD_MESH"), 1);
#if 0
	const bool bUseGPUSceneAndPrimitiveIdStream =
		Parameters.VertexFactoryType->SupportsPrimitiveIdStream() 
		&& UseGPUScene(Parameters.Platform, GetMaxSupportedFeatureLevel(Parameters.Platform))
		// TODO: support GPUScene on mobile
		&& !IsMobilePlatform(Parameters.Platform);
		
	OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), bUseGPUSceneAndPrimitiveIdStream);
#endif
}

void FVirtualHeightfieldMeshVertexFactory::ValidateCompiledResult(const FVertexFactoryType* Type, EShaderPlatform Platform, const FShaderParameterMap& ParameterMap, TArray<FString>& OutErrors)
{
#if 0
	if (Type->SupportsPrimitiveIdStream()
		&& UseGPUScene(Platform, GetMaxSupportedFeatureLevel(Platform))
		&& ParameterMap.ContainsParameterAllocation(FPrimitiveUniformShaderParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName()))
	{
		OutErrors.AddUnique(*FString::Printf(TEXT("Shader attempted to bind the Primitive uniform buffer even though Vertex Factory %s computes a PrimitiveId per-instance.  This will break auto-instancing.  Shaders should use GetPrimitiveData(Parameters).Member instead of Primitive.Member."), Type->GetName()));
	}
#endif
}

IMPLEMENT_VERTEX_FACTORY_TYPE(FVirtualHeightfieldMeshVertexFactory, "/Plugin/VirtualHeightfieldMesh/Private/VirtualHeightfieldMeshVertexFactory.ush",
	  EVertexFactoryFlags::UsedWithMaterials
	| EVertexFactoryFlags::SupportsDynamicLighting
	| EVertexFactoryFlags::SupportsPrimitiveIdStream
);
