// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VertexFactory.h"
#include "SceneManagement.h"

class FShaderParameterMap;
struct FShaderCompilerEnvironment;

/**
 * Uniform buffer to hold parameters specific to this vertex factory. Only set up once.
 */
BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FVirtualHeightfieldMeshVertexFactoryParameters, )
	SHADER_PARAMETER_TEXTURE(Texture2D<uint4>, PageTableTexture)
	SHADER_PARAMETER_SRV(Texture2D<float>, HeightTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, HeightSampler)
	SHADER_PARAMETER_TEXTURE(Texture2D<float>, LodBiasTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, LodBiasSampler)
	SHADER_PARAMETER(int32, NumQuadsPerTileSide)
	SHADER_PARAMETER(FUintVector4, VTPackedUniform)
	SHADER_PARAMETER(FUintVector4, VTPackedPageTableUniform0)
	SHADER_PARAMETER(FUintVector4, VTPackedPageTableUniform1)
	SHADER_PARAMETER(FVector4f, PageTableSize)
	SHADER_PARAMETER(FVector2f, PhysicalTextureSize)
	SHADER_PARAMETER(FMatrix44f, VirtualHeightfieldToLocal)
	SHADER_PARAMETER(FMatrix44f, VirtualHeightfieldToWorld)
	SHADER_PARAMETER(float, MaxLod)
	SHADER_PARAMETER(float, LodBiasScale)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

typedef TUniformBufferRef<FVirtualHeightfieldMeshVertexFactoryParameters> FVirtualHeightfieldMeshVertexFactoryBufferRef;

/**
 * Per frame UserData to pass to the vertex shader.
 */
struct FVirtualHeightfieldMeshUserData : public FOneFrameResource
{
	FRHIShaderResourceView* InstanceBufferSRV;
	FVector3f LodViewOrigin;
	FVector4f LodDistances;
};

/**
 * Index buffer for a single Virtual Heightfield Mesh tile.
 */
class FVirtualHeightfieldMeshIndexBuffer : public FIndexBuffer
{
public:

	FVirtualHeightfieldMeshIndexBuffer(int32 InNumQuadsPerSide) 
		: NumQuadsPerSide(InNumQuadsPerSide) 
	{}

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	int32 GetIndexCount() const { return NumIndices; }

private:
	int32 NumIndices = 0;
	const int32 NumQuadsPerSide = 0;
};

/**
 * Virtual Heightfield Mesh vertex factory.
 */
class FVirtualHeightfieldMeshVertexFactory : public FVertexFactory
{
	DECLARE_VERTEX_FACTORY_TYPE(FVirtualHeightfieldMeshVertexFactory);

public:
	FVirtualHeightfieldMeshVertexFactory(ERHIFeatureLevel::Type InFeatureLevel, const FVirtualHeightfieldMeshVertexFactoryParameters& InParams);

	~FVirtualHeightfieldMeshVertexFactory();

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseRHI() override;

	static bool ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
	static void ValidateCompiledResult(const FVertexFactoryType* Type, EShaderPlatform Platform, const FShaderParameterMap& ParameterMap, TArray<FString>& OutErrors);

	FIndexBuffer const* GetIndexBuffer() const { return IndexBuffer; }

private:
	FVirtualHeightfieldMeshVertexFactoryParameters Params;
	FVirtualHeightfieldMeshVertexFactoryBufferRef UniformBuffer;
	FVirtualHeightfieldMeshIndexBuffer* IndexBuffer = nullptr;

	friend class FVirtualHeightfieldMeshVertexFactoryShaderParameters;
};
