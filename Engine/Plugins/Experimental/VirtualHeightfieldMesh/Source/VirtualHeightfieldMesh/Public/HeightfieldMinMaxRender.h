// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"

class FRDGBuilder;
class FRDGTexture;
class FRDGTextureUAV;

namespace VirtualHeightfieldMesh
{
	/**
	 * Utility funtion to downsample a height texture and then pack and write the MinMax values to a texel in the destination texture.
	 * SrcTexture is expected to by G16 and DstTexture is expected to be RGBA8 packed as 16 bit min and max split across the 8 bit channels.
	 */
	VIRTUALHEIGHTFIELDMESH_API void DownsampleMinMaxAndCopy(FRDGBuilder& GraphBuilder, FRDGTexture* SrcTexture, FIntPoint SrcSize, FRDGTextureUAV* DstTexture, FIntPoint DstCoord);

	/** Utility function to generate all additional mips from mip0 for a MinMax height texture already packed in RGBA8. */
	VIRTUALHEIGHTFIELDMESH_API void GenerateMinMaxTextureMips(FRDGBuilder& GraphBuilder, FRDGTexture* Texture, FIntPoint SrcSize, int32 NumMips);
};
