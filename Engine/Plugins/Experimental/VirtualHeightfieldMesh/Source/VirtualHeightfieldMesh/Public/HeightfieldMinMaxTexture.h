// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HeightfieldMinMaxTexture.generated.h"

#define UE_API VIRTUALHEIGHTFIELDMESH_API

struct FPropertyChangedEvent;

#if WITH_EDITOR

/** Description object used to build the contents of a UHeightfieldMinMaxTexture. */
struct FHeightfieldMinMaxTextureBuildDesc
{
	uint32 SizeX = 0;
	uint32 SizeY = 0;
	uint32 NumMips = 0;
	uint8 const* Data = nullptr;
};

#endif

/**
 * Container for a UTexture2D that can be built from a FHeightfieldMinMaxTextureBuildDesc description.
 */
UCLASS(MinimalAPI, ClassGroup = Rendering, BlueprintType)
class UHeightfieldMinMaxTexture : public UObject
{
public:
	GENERATED_UCLASS_BODY()

	/** The Height MinMax texture. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = Texture, meta = (DisplayName = "Height MinMax Texture"))
	TObjectPtr<class UTexture2D> Texture = nullptr;

	/** A LodBias texture derived from the Height MinMax texture*/
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = Texture, meta = (DisplayName = "LodBias Texture"))
	TObjectPtr<class UTexture2D> LodBiasTexture = nullptr;

	/** A LodBias MinMax texture derived from the LodBias texture*/
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = Texture, meta = (DisplayName = "LodBias MinMax Texture"))
	TObjectPtr<class UTexture2D> LodBiasMinMaxTexture = nullptr;

protected:
	/** The number of mip levels to clone for CPU access. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Texture, meta = (DisplayName = "Num CPU Levels", UIMin = "0", ClampMin = "0"))
	int32 MaxCPULevels;

public:
	/** The raw MinMax data from the low resolution mip levels of Texture. These are CPU access of the MinMax bounds. */
	UPROPERTY()
	TArray<FVector2D> TextureData;

	/** The size of the largest mip stored in TextureData. */
	UPROPERTY()
	FIntPoint TextureDataSize;

	/** The starting array index for the data of each mip stored in TextureData. */
	UPROPERTY()
	TArray<int32> TextureDataMips;

#if WITH_EDITOR
	/** Creates a new UTexture2D and stores it in the contained Texture. */
	UE_API void BuildTexture(FHeightfieldMinMaxTextureBuildDesc const& InBuildDesc);
#endif
 
protected:
#if WITH_EDITOR
	/** Rebuild the contents of Texture using the passed in build description. */
	UE_API void RebuildMinMaxTexture(FHeightfieldMinMaxTextureBuildDesc const& InBuildDesc);
	/** Rebuild the contents of LodBiasTexture from the MinMaxTexture. */
	UE_API void RebuildLodBiasTexture(FHeightfieldMinMaxTextureBuildDesc const& InBuildDesc);
	/** Rebuild the contents of TextureData for the specified number of CPU mip levels. */
	UE_API void RebuildCPUTextureData();

	//~ Begin UObject Interface
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface
#endif
};

#undef UE_API
