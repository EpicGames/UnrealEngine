// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "VirtualHeightfieldMeshActor.generated.h"

UCLASS(hidecategories = (Actor, Collision, Cooking, Input, LOD, Replication), MinimalAPI)
class AVirtualHeightfieldMesh : public AActor
{
	GENERATED_UCLASS_BODY()

private:
	/** Component for rendering the big mesh. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = VirtualTexture, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<class UVirtualHeightfieldMeshComponent> VirtualHeightfieldMeshComponent;

#if WITH_EDITORONLY_DATA
	/** Box for visualizing virtual texture extents. */
	UPROPERTY(Transient)
	TObjectPtr<class UBoxComponent> Box = nullptr;
#endif // WITH_EDITORONLY_DATA

protected:
	//~ Begin UObject Interface.
	virtual bool NeedsLoadForServer() const override { return false; }
	//~ End UObject Interface.
};
