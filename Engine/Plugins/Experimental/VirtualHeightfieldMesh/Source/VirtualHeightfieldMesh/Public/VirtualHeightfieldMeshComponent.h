// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/PrimitiveComponent.h"
#include "Physics/PhysicsInterfaceTypes.h"
#include "VirtualHeightfieldMeshComponent.generated.h"

#define UE_API VIRTUALHEIGHTFIELDMESH_API

class ARuntimeVirtualTextureVolume;
class UHeightfieldMinMaxTexture;
class UMaterialInterface;
class URuntimeVirtualTexture;

/** Component to render a heightfield mesh using a virtual texture heightmap. */
UCLASS(MinimalAPI, Blueprintable, ClassGroup = Rendering, hideCategories = (Activation, Collision, Cooking, HLOD, Navigation, Mobility, Object, Physics, VirtualTexture))
class UVirtualHeightfieldMeshComponent : public UPrimitiveComponent
{
	GENERATED_UCLASS_BODY()

protected:
	/** The RuntimeVirtualTextureVolume that contains virtual texture heightmap. */
	UPROPERTY(EditAnywhere, Category = Heightfield)
	TSoftObjectPtr<ARuntimeVirtualTextureVolume> VirtualTexture;
	
	/** UObject ref resolved from VirtualTexture weak ref. */
	UPROPERTY(Transient)
	TObjectPtr<ARuntimeVirtualTextureVolume> VirtualTextureRef;

	/** Placeholder for details customization image. */
	UPROPERTY(VisibleAnywhere, Transient, Category = Heightfield)
	TObjectPtr<UObject> VirtualTextureThumbnail = nullptr;

	/** Placeholder for details customization button. */
	UPROPERTY(VisibleAnywhere, Transient, Category = Heightfield)
	bool bCopyBoundsButton;

	/** Texture object containing minimum and maximum height values. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = HeightfieldBuild, meta = (DisplayName = "MinMax Texture"))
	TObjectPtr<UHeightfieldMinMaxTexture> MinMaxTexture = nullptr;

	/** Number of levels to build in the MinMax Texture. A default value of 0 will build all levels from the heightfield. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = HeightfieldBuild, meta = (DisplayName = "MinMax Build Levels", UIMin = "0"))
	int32 NumMinMaxTextureBuildLevels = 0;

	/** Placeholder for details customization button. */
	UPROPERTY(VisibleAnywhere, Transient, Category = HeightfieldBuild)
	bool bBuildMinMaxTextureButton;

	/** The material to apply. */
	UPROPERTY(EditAnywhere, Category = Rendering)
	TObjectPtr<UMaterialInterface> Material = nullptr;

	/** Target screen size for a LOD 0 tile. A smaller value uniformly increases the geometry resolution on screen. */
	UPROPERTY(EditAnywhere, Category = Rendering, meta = (DisplayName = "LOD 0 Screen Size", ClampMin = "0.1", UIMin = "0.1"))
	float Lod0ScreenSize = 1.f;

	/** Distribution multiplier applied only for LOD 0. A larger value increases the distance to the first LOD transition. */
	UPROPERTY(EditAnywhere, Category = Rendering, meta = (DisplayName = "LOD 0 Distance Scale", ClampMin = "1.0", UIMin = "1.0"))
	float Lod0Distribution = 1.f;

	/** Distribution multiplier applied for each LOD level. A larger value increases the distance exponentially between each LOD transition. */
	UPROPERTY(EditAnywhere, Category = Rendering, meta = (DisplayName = "LOD Distribution", ClampMin = "1.0", UIMin = "1.0", UIMax = "3.0"))
	float LodDistribution = 2.f;

	/** Scale applied to LodBias texture. A larger value increases resolution where the LodBias texture is set. */
	UPROPERTY(EditAnywhere, Category = Rendering, meta = (DisplayName = "LOD Bias Scale", ClampMin = "0.0", ClampMax = "10.0", UIMin = "0.0", UIMax = "10.0"))
	float LodBiasScale = 0.f;

	/** The number of levels that we force loaded. A higher number can reduce LOD pop in but allocates more of the virtual texture memory pool. */
	UPROPERTY(EditAnywhere, Category = Rendering, meta = (DisplayName = "Force Load LODs", ClampMin = "0", ClampMax = "4"))
	int32 NumForceLoadLods = 0;

	/** The number of levels that we calculate occlusion volumes for. A higher number gives finer occlusion at the cost of more queries. */
	UPROPERTY(EditAnywhere, Category = Rendering, meta = (DisplayName = "Occlusion LODs", ClampMin = "0", ClampMax = "5"))
	int32 NumOcclusionLods = 0;

	/** Allows us to only see this actor in game and not in the Editor. This is useful if we only want to see the Heightfield virtual texture source primitives during edition. */
	UPROPERTY(EditAnywhere, Category = Rendering, meta = (DisplayName = "Actor Hidden In Editor", DisplayAfter = "bHiddenInGame"))
	bool bHiddenInEditor = true;

	/** Allows material World Position Offset to contribute to velocity rendering.*/
	UPROPERTY(EditAnywhere, Category = Rendering, AdvancedDisplay)
	bool bWorldPositionOffsetVelocity = false;

public:
	/** Get the HiddenInEditor flag  on this component. */
	bool GetHiddenInEditor() const { return bHiddenInEditor; }

	/** Get the associated runtime virtual texture volume. Can return nullptr if the volume is from an unloaded level. */
	UE_API ARuntimeVirtualTextureVolume* GetVirtualTextureVolume() const;
	/** Get the associated runtime virtual texture transform including any texel snap offset. */
	UE_API FTransform GetVirtualTextureTransform() const;
	/** Get the associated runtime virtual texture. */
	UE_API URuntimeVirtualTexture* GetVirtualTexture() const;
	
	/** Returns true if a MinMax height texture is relevant for this virtual texture type. */
	UE_API bool IsMinMaxTextureEnabled() const;
	/** Get the MinMax height texture on this component. */
	UHeightfieldMinMaxTexture* GetMinMaxTexture() const { return MinMaxTexture; }
	/** Get the number of levels to build in the MinMax Texture. */
	int32 GetNumMinMaxTextureBuildLevels() { return NumMinMaxTextureBuildLevels; }

#if WITH_EDITOR
	/** Set a new asset to hold the MinMax height texture. This should only be called directly before setting data to the new asset. */
	void SetMinMaxTexture(UHeightfieldMinMaxTexture* InTexture) { MinMaxTexture = InTexture; }
	/** Initialize the MinMax height texture with the passed in size and data. */
	UE_API void InitializeMinMaxTexture(uint32 InSizeX, uint32 InSizeY, uint32 InNumMips, uint8* InData);
#endif

	UMaterialInterface* GetMaterial() const { return Material; }
	float GetLod0ScreenSize() const { return Lod0ScreenSize; }
	float GetLod0Distribution() const { return Lod0Distribution; }
	float GetLodDistribution() const { return LodDistribution; }
	float GetLodBiasScale() const { return LodBiasScale; }
	int32 GetNumForceLoadLods() const { return NumForceLoadLods; }
	int32 GetNumOcclusionLods() const { return NumOcclusionLods; }

protected:
	/** Function used by the VirtualTexture delegate to retrieve our HidePrimitives flags. */
	UFUNCTION()
	UE_API void GatherHideFlags(bool& InOutHidePrimitivesInEditor, bool& InOutHidePrimitivesInGame) const;

	/** Handles when the transform is updated on the associated VirtualTexture. */
	UE_API void OnVirtualTextureTransformUpdate(USceneComponent* InRootComponent, EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport);

	//~ Begin UObject Interface.
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject Interface.

	//~ Begin UActorComponent Interface
	UE_API virtual void OnRegister() override;
	UE_API virtual void OnUnregister() override;
	UE_API virtual void ApplyWorldOffset(const FVector& InOffset, bool bWorldShift) override;
	//~ End UActorComponent Interface

	//~ Begin USceneComponent Interface
	UE_API virtual bool IsVisible() const override;
	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	//~ EndUSceneComponent Interface

	//~ Begin UPrimitiveComponent Interface
	UE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	virtual bool SupportsStaticLighting() const override { return true; }
	UE_API virtual void SetMaterial(int32 ElementIndex, class UMaterialInterface* Material) override;
	virtual UMaterialInterface* GetMaterial(int32 Index) const override { return Material; }
	UE_API virtual void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials = false) const override;
	virtual bool SupportsWorldPositionOffsetVelocity() const override { return bWorldPositionOffsetVelocity; }
	//~ End UPrimitiveComponent Interface
};

#undef UE_API
