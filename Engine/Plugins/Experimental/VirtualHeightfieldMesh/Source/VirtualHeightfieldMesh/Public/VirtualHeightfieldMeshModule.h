// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

/** Implements the VirtualHeightfieldMesh module  */
class FVirtualHeightfieldMeshModule : public IModuleInterface
{
public:
	//~ Begin IModuleInterface Interface.
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface Interface.
};
