// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class VirtualHeightfieldMesh : ModuleRules
{
	public VirtualHeightfieldMesh(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(new string[] {
			"Core",
			"CoreUObject",
			"Engine",
			"Projects",
			"RenderCore",
			"Renderer",
			"RHI",
		});
	}
}
