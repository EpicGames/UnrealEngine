// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeightfieldMinMaxTextureAssetTypeActions.h"

#include "HeightfieldMinMaxTexture.h"

#define LOCTEXT_NAMESPACE "VirtualHeightfieldMeshEditorModule"

UClass* FAssetTypeActions_HeightfieldMinMaxTexture::GetSupportedClass() const
{
	return UHeightfieldMinMaxTexture::StaticClass();
}

FText FAssetTypeActions_HeightfieldMinMaxTexture::GetName() const
{
	return LOCTEXT("AssetTypeActions_HeightfieldMinMaxTextureBuilder", "Heightfield MinMax Texture"); 
}

FColor FAssetTypeActions_HeightfieldMinMaxTexture::GetTypeColor() const
{
	return FColor(128, 128, 128); 
}

uint32 FAssetTypeActions_HeightfieldMinMaxTexture::GetCategories()
{
	return EAssetTypeCategories::Textures; 
}

#undef LOCTEXT_NAMESPACE
