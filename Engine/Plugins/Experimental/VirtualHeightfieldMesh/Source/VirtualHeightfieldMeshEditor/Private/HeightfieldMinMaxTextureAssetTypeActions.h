// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetTypeActions_Base.h"

class UHeightfieldMinMaxTexture;

/** Asset actions setup for UHeightfieldMinMaxTexture */
class FAssetTypeActions_HeightfieldMinMaxTexture : public FAssetTypeActions_Base
{
public:
	FAssetTypeActions_HeightfieldMinMaxTexture() {}

protected:
	//~ Begin IAssetTypeActions Interface.
	virtual UClass* GetSupportedClass() const override;
	virtual FText GetName() const override;
	virtual FColor GetTypeColor() const override;
	virtual uint32 GetCategories() override;
	//~ End IAssetTypeActions Interface.
};
