// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeightfieldMinMaxTextureBuild.h"

#include "Components/RuntimeVirtualTextureComponent.h"
#include "ContentStreaming.h"
#include "EngineModule.h"
#include "HeightfieldMinMaxRender.h"
#include "HeightfieldMinMaxTexture.h"
#include "Misc/ScopedSlowTask.h"
#include "RenderGraphBuilder.h"
#include "RenderTargetPool.h"
#include "RenderingThread.h"
#include "SceneInterface.h"
#include "VirtualHeightfieldMeshComponent.h"
#include "VT/RuntimeVirtualTexture.h"
#include "VT/RuntimeVirtualTextureRender.h"
#include "VT/RuntimeVirtualTextureVolume.h"

namespace
{
	/** Container for render resources needed to render the MinMax height texture. */
	class FMinMaxTileRenderResources : public FRenderResource
	{
	public:
		FMinMaxTileRenderResources(int32 InTileSize, int32 InNumTilesX, int32 InNumTilesY, int32 InNumMips)
			: TileSize(InTileSize)
			, NumTilesX(InNumTilesX)
			, NumTilesY(InNumTilesY)
			, NumMips(InNumMips)
			, NumFinalTexels(0)
		{
			for (int32 MipLevel = 0; MipLevel < NumMips; MipLevel++)
			{
				NumFinalTexels += FMath::Max(NumTilesX >> MipLevel, 1) * FMath::Max(NumTilesY >> MipLevel, 1);
			}
		}

		//~ Begin FRenderResource Interface.
		virtual void InitRHI(FRHICommandListBase& RHICmdList) override
		{
			FPooledRenderTargetDesc TileRenderTargetDesc = FPooledRenderTargetDesc::Create2DDesc(
				FIntPoint(TileSize, TileSize),	PF_G16,	FClearValueBinding::None, TexCreate_None, TexCreate_ShaderResource, false);
			GRenderTargetPool.FindFreeElement(RHICmdList, TileRenderTargetDesc, TileRenderTarget, TEXT("TileTarget"));
			
			FPooledRenderTargetDesc FinalRenderTargetDesc = FPooledRenderTargetDesc::Create2DDesc(
				FIntPoint(NumTilesX, NumTilesY), PF_R8G8B8A8, FClearValueBinding::None, TexCreate_None, TexCreate_UAV | TexCreate_ShaderResource | TexCreate_RenderTargetable, false, NumMips);
			GRenderTargetPool.FindFreeElement(RHICmdList, FinalRenderTargetDesc, FinalRenderTarget, TEXT("FinalTarget"));

			for (int32 MipLevel = 0; MipLevel < NumMips; MipLevel++)
			{
				const int32 SizeX = FMath::Max(NumTilesX >> MipLevel, 1);
				const int32 SizeY = FMath::Max(NumTilesY >> MipLevel, 1);

				const FRHITextureCreateDesc Desc =
					FRHITextureCreateDesc::Create2D(TEXT("FMinMaxTileRenderResources_StagingTextures"), SizeX, SizeY, PF_R8G8B8A8)
					.SetFlags(ETextureCreateFlags::CPUReadback);

				StagingTextures.Add(RHICmdList.CreateTexture(Desc));
			}
			
			Fence = RHICreateGPUFence(TEXT("Runtime Virtual Texture Build"));
		}

		virtual void ReleaseRHI() override
		{
			TileRenderTarget.SafeRelease();
			FinalRenderTarget.SafeRelease();
			
			for (int32 MipLevel = 0; MipLevel < NumMips; MipLevel++)
			{
				StagingTextures[MipLevel].SafeRelease();
			}

			Fence.SafeRelease();
		}
		//~ End FRenderResource Interface.

		int32 GetNumFinalTexels() const { return NumFinalTexels; }
		TRefCountPtr<IPooledRenderTarget> GetTileRenderTarget() const { return TileRenderTarget; }
		TRefCountPtr<IPooledRenderTarget> GetFinalRenderTarget() const { return FinalRenderTarget; }
		FTextureRHIRef GetStagingTexture(int32 InMipLevel) const { return StagingTextures[InMipLevel]; }
		FRHIGPUFence* GetFence() const { return Fence; }

	private:
		int32 TileSize;
		int32 NumTilesX;
		int32 NumTilesY;
		int32 NumMips;
		int32 NumFinalTexels;

		TRefCountPtr<IPooledRenderTarget> TileRenderTarget;
		TRefCountPtr<IPooledRenderTarget> FinalRenderTarget;
		TArray<FTextureRHIRef> StagingTextures;
		FGPUFenceRHIRef Fence;
	};
}

namespace VirtualHeightfieldMesh
{
	bool HasMinMaxHeightTexture(UVirtualHeightfieldMeshComponent* InComponent)
	{
		if (InComponent == nullptr || !InComponent->IsMinMaxTextureEnabled() || InComponent->GetMinMaxTexture() == nullptr)
		{
			return false;
		}
		
		if (InComponent->GetVirtualTextureVolume() == nullptr || InComponent->GetVirtualTextureVolume()->VirtualTextureComponent == nullptr)
		{
			return false;
		}

		return true;
	}

	bool BuildMinMaxHeightTexture(UVirtualHeightfieldMeshComponent* InComponent)
	{
		if (!HasMinMaxHeightTexture(InComponent))
		{
			return true;
		}

		ARuntimeVirtualTextureVolume* VirtualTextureVolume = InComponent->GetVirtualTextureVolume();
		URuntimeVirtualTextureComponent* VirtualTextureComponent = VirtualTextureVolume != nullptr ? ToRawPtr(VirtualTextureVolume->VirtualTextureComponent) : nullptr;

		if (VirtualTextureComponent == nullptr)
		{
			return false;
		}

		FSceneInterface* Scene = VirtualTextureComponent->GetScene();
		const FTransform Transform = VirtualTextureComponent->GetComponentTransform();
		const FBox Bounds = VirtualTextureComponent->Bounds.GetBox();

		URuntimeVirtualTexture* VirtualTexture = InComponent->GetVirtualTexture();
		const int32 RuntimeVirtualTextureId = VirtualTexture->GetUniqueID();

		FVTProducerDescription VTDesc;
		VirtualTexture->GetProducerDescription(VTDesc, URuntimeVirtualTexture::FInitSettings(), Transform);

		const int32 TileSize = VTDesc.TileSize;
		const int32 MaxLevel = VTDesc.MaxLevel;

		// Adjust number of tiles to match requested number of build levels
		const int32 NumBuildLevels = InComponent->GetNumMinMaxTextureBuildLevels();
		const int32 IncLevels = NumBuildLevels == 0 ? 0 : FMath::Max(NumBuildLevels - MaxLevel - 1, 0);
		const int32 DecLevels = NumBuildLevels == 0 ? 0 : FMath::Max(MaxLevel - NumBuildLevels + 1, 0);
		const int32 NumTilesX = ((VTDesc.WidthInBlocks * VTDesc.BlockWidthInTiles) << IncLevels) >> DecLevels;
		const int32 NumTilesY = ((VTDesc.WidthInBlocks * VTDesc.BlockWidthInTiles) << IncLevels) >> DecLevels;
		const int32 NumMips = (int32)FMath::CeilLogTwo(FMath::Max(NumTilesX, NumTilesY)) + 1;

		// Allocate render targets for rendering out the runtime virtual texture tiles
		FMinMaxTileRenderResources RenderTileResources(TileSize, NumTilesX, NumTilesY, NumMips);
		BeginInitResource(&RenderTileResources);

		// Spin up slow task UI
		const float TaskWorkRender = NumTilesX * NumTilesY;
		const float TaskWorkDownsample = 2;
		const float TaskWorkBuildBulkData = 2;
		FScopedSlowTask Task(TaskWorkRender + TaskWorkDownsample + TaskWorkBuildBulkData, FText::AsCultureInvariant(InComponent->GetMinMaxTexture()->GetName()));
		Task.MakeDialog(true);

		// Final pixels will contain image data for MinMax texture
		TArray64<uint8> FinalPixels;
		FinalPixels.SetNumUninitialized(RenderTileResources.GetNumFinalTexels() * 4);

		// Iterate over all mip0 tiles and downsample/store each one to the final image
		for (int32 TileY = 0; TileY < NumTilesY && !Task.ShouldCancel(); TileY++)
		{
			for (int32 TileX = 0; TileX < NumTilesX; TileX++)
			{
				// Render tile
				Task.EnterProgressFrame();

				const FBox2D UVRange = FBox2D(
					FVector2D((float)TileX / (float)NumTilesX, (float)TileY / (float)NumTilesY),
					FVector2D((float)(TileX + 1) / (float)NumTilesX, (float)(TileY + 1) / (float)NumTilesY));

				// Stream textures for this tile. This triggers a render flush internally.
				//todo[vt]: Batch groups of streaming locations and render commands to reduce number of flushes.
				const FVector StreamingWorldPos = Transform.TransformPosition(FVector(UVRange.GetCenter(), 0.5f));
				IStreamingManager::Get().Tick(0.f);
				IStreamingManager::Get().AddViewLocation(StreamingWorldPos);
				IStreamingManager::Get().StreamAllResources(0);

				ENQUEUE_RENDER_COMMAND(MinMaxTextureTileCommand)([
					Scene, RuntimeVirtualTextureId,
					&RenderTileResources,
					Transform, Bounds, UVRange,
					TileX, TileY, TileSize,
					MaxLevel, MipLevel = 0](FRHICommandListImmediate& RHICmdList)
				{
					// Rendering one page at a time, but could batch here?
					FRDGBuilder GraphBuilder(RHICmdList);
					FScenePrimitiveRenderingContextScopeHelper RenderingScope(GetRendererModule().BeginScenePrimitiveRendering(GraphBuilder, *Scene));

					RuntimeVirtualTexture::FRenderPageBatchDesc Desc;
					Desc.SceneRenderer = RenderingScope.ScenePrimitiveRenderingContext->GetSceneRenderer();
					Desc.RuntimeVirtualTextureId = RuntimeVirtualTextureId;
					Desc.UVToWorld = Transform;
					Desc.WorldBounds = Bounds;
					Desc.MaterialType = ERuntimeVirtualTextureMaterialType::WorldHeight;
					Desc.MaxLevel = MaxLevel;
					Desc.bClearTextures = true;
					Desc.bIsThumbnails = false;
					Desc.FixedColor = FLinearColor::Transparent;
					Desc.NumPageDescs = 1;
					Desc.Targets[0].PooledRenderTarget = RenderTileResources.GetTileRenderTarget();
					Desc.Targets[1].PooledRenderTarget = nullptr;
					Desc.Targets[2].PooledRenderTarget = nullptr;
					Desc.PageDescs[0].DestRect[0] = FIntRect(0, 0, TileSize, TileSize);
					Desc.PageDescs[0].DestRect[1] = FIntRect();
					Desc.PageDescs[0].DestRect[2] = FIntRect();
					Desc.PageDescs[0].UVRange = UVRange;
					Desc.PageDescs[0].vLevel = MipLevel;

					RenderPages(GraphBuilder, Desc);

					// Downsample page to texel in output
					FRDGTextureRef SrcTexture = GraphBuilder.RegisterExternalTexture(RenderTileResources.GetTileRenderTarget());
					FRDGTextureRef DstTexture = GraphBuilder.RegisterExternalTexture(RenderTileResources.GetFinalRenderTarget());
					FRDGTextureUAVRef DstTextureUAV = GraphBuilder.CreateUAV(DstTexture);
					
					DownsampleMinMaxAndCopy(GraphBuilder, SrcTexture, FIntPoint(TileSize, TileSize), DstTextureUAV, FIntPoint(TileX, TileY));
					
					GraphBuilder.Execute();
				});
			}
		}

		// Downsample and copy to staging
		if (!Task.ShouldCancel())
		{
			Task.EnterProgressFrame(TaskWorkDownsample);

			ENQUEUE_RENDER_COMMAND(MinMaxTextureTileCommand)([&RenderTileResources, &FinalPixels, NumTilesX, NumTilesY, NumMips](FRHICommandListImmediate& RHICmdList)
			{
				FRDGBuilder GraphBuilder(RHICmdList);

				FRDGTextureRef Texture = GraphBuilder.RegisterExternalTexture(RenderTileResources.GetFinalRenderTarget());
				GraphBuilder.SetTextureAccessFinal(Texture, ERHIAccess::CopySrc);

				GenerateMinMaxTextureMips(GraphBuilder, Texture, FIntPoint(NumTilesX, NumTilesY), NumMips);

				GraphBuilder.Execute();

				for (int32 MipLevel = 0; MipLevel < NumMips; MipLevel++)
				{
					FRHICopyTextureInfo CopyInfo;
					CopyInfo.Size = FIntVector(RenderTileResources.GetStagingTexture(MipLevel)->GetSizeXYZ());
					CopyInfo.SourceMipIndex = MipLevel;
					CopyInfo.DestMipIndex = 0;

					RHICmdList.Transition(FRHITransitionInfo(RenderTileResources.GetStagingTexture(MipLevel), ERHIAccess::Unknown, ERHIAccess::CopyDest));
					RHICmdList.CopyTexture(RenderTileResources.GetFinalRenderTarget()->GetRHI(), RenderTileResources.GetStagingTexture(MipLevel), CopyInfo);
				}

				RHICmdList.WriteGPUFence(RenderTileResources.GetFence());

				uint8* WritePtr = FinalPixels.GetData();
				for (int32 MipLevel = 0; MipLevel < NumMips; MipLevel++)
				{
					void* TilePixels = nullptr;
					int32 OutWidth, OutHeight;
					RHICmdList.MapStagingSurface(RenderTileResources.GetStagingTexture(MipLevel), RenderTileResources.GetFence(), TilePixels, OutWidth, OutHeight);
					check(TilePixels != nullptr);

					int32 Width = RenderTileResources.GetStagingTexture(MipLevel)->GetSizeXY().Y;
					uint8* ReadPtr = (uint8*)TilePixels;
					for (int32 Y = 0; Y < OutHeight; ++Y)
					{
						FMemory::Memcpy(WritePtr, ReadPtr, Width * 4);
						WritePtr += Width * 4;
						ReadPtr += OutWidth * 4;
					}

					RHICmdList.UnmapStagingSurface(RenderTileResources.GetStagingTexture(MipLevel));
				}

				check(WritePtr - FinalPixels.GetData() == FinalPixels.Num());
			});

			FlushRenderingCommands();
		}

		BeginReleaseResource(&RenderTileResources);
		FlushRenderingCommands();

		if (Task.ShouldCancel())
		{
			return false;
		}

		// Build final texture
		Task.EnterProgressFrame(TaskWorkBuildBulkData);

		InComponent->InitializeMinMaxTexture(NumTilesX, NumTilesY, NumMips, FinalPixels.GetData());

		return true;
	}
}
