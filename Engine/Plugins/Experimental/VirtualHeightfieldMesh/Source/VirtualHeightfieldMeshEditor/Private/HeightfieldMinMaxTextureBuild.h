// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


class UVirtualHeightfieldMeshComponent;

namespace VirtualHeightfieldMesh
{
	/** Returns true if the component has a MinMax height texture. */
	bool HasMinMaxHeightTexture(UVirtualHeightfieldMeshComponent* InComponent);

	/** Build the MinMax height texture. */
	bool BuildMinMaxHeightTexture(UVirtualHeightfieldMeshComponent* InComponent);
};
