// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeightfieldMinMaxTextureFactory.h"

#include "HeightfieldMinMaxTexture.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HeightfieldMinMaxTextureFactory)

UHeightfieldMinMaxTextureFactory::UHeightfieldMinMaxTextureFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UHeightfieldMinMaxTexture::StaticClass();
	
	bCreateNew = true;
	bEditAfterNew = false;
	bEditorImport = false;
}

UObject* UHeightfieldMinMaxTextureFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UHeightfieldMinMaxTexture>(InParent, Class, Name, Flags);
}

