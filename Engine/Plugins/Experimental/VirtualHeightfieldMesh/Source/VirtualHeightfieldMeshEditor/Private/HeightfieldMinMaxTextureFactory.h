// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "HeightfieldMinMaxTextureFactory.generated.h"

/** Factory for UHeightfieldMinMaxTexture */
UCLASS(hidecategories = (Object))
class UHeightfieldMinMaxTextureFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	virtual bool ShouldShowInNewMenu() const { return false; }
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface	
};
