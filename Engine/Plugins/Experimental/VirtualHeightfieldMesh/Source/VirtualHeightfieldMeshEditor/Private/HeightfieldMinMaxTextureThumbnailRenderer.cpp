// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeightfieldMinMaxTextureThumbnailRenderer.h"

#include "Engine/Texture2D.h"
#include "HeightfieldMinMaxTexture.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HeightfieldMinMaxTextureThumbnailRenderer)

UHeightfieldMinMaxTextureThumbnailRenderer::UHeightfieldMinMaxTextureThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UHeightfieldMinMaxTextureThumbnailRenderer::CanVisualizeAsset(UObject* Object)
{
	if (UHeightfieldMinMaxTexture* MinMaxTextureBuilder = Cast<UHeightfieldMinMaxTexture>(Object))
	{
		if (UTexture2D* Texture = MinMaxTextureBuilder->Texture.Get())
		{
			return UTextureThumbnailRenderer::CanVisualizeAsset(Texture);
		}
	}
	return false;
}

void UHeightfieldMinMaxTextureThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	if (UHeightfieldMinMaxTexture* MinMaxTextureBuilder = Cast<UHeightfieldMinMaxTexture>(Object))
	{
		if (UTexture2D* Texture = MinMaxTextureBuilder->Texture)
		{
			UTextureThumbnailRenderer::Draw(Texture, X, Y, Width, Height, RenderTarget, Canvas, bAdditionalViewFamily);
		}
	}
}

