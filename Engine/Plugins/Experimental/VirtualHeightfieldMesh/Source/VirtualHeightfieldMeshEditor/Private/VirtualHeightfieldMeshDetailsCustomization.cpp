// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualHeightfieldMeshDetailsCustomization.h"

#include "AssetThumbnail.h"
#include "AssetToolsModule.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "HeightfieldMinMaxTexture.h"
#include "HeightfieldMinMaxTextureBuild.h"
#include "HeightfieldMinMaxTextureFactory.h"
#include "IAssetTools.h"
#include "ScopedTransaction.h"
#include "Misc/PackageName.h"
#include "VirtualHeightfieldMeshComponent.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "VT/RuntimeVirtualTexture.h"
#include "VT/RuntimeVirtualTextureVolume.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "VirtualHeightfieldMeshEditorModule"

FVirtualHeightfieldMeshComponentDetailsCustomization::FVirtualHeightfieldMeshComponentDetailsCustomization()
	: AssetThumbnail(MakeShared<FAssetThumbnail>(nullptr, 64, 64, UThumbnailManager::Get().GetSharedThumbnailPool()))
{
}

TSharedRef<IDetailCustomization> FVirtualHeightfieldMeshComponentDetailsCustomization::MakeInstance()
{
	return MakeShareable(new FVirtualHeightfieldMeshComponentDetailsCustomization);
}

void FVirtualHeightfieldMeshComponentDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// Get and store the linked UVirtualHeightfieldMeshComponent
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	if (ObjectsBeingCustomized.Num() > 1)
	{
		return;
	}
	VirtualHeightfieldMeshComponent = Cast<UVirtualHeightfieldMeshComponent>(ObjectsBeingCustomized[0].Get());
	if (VirtualHeightfieldMeshComponent == nullptr)
	{
		return;
	}

	// Apply custom widget for VirtualTextureThumbnail
	TSharedRef<IPropertyHandle> VirtualTextureThumbnailPropertyHandle = DetailBuilder.GetProperty(TEXT("VirtualTextureThumbnail"));
	DetailBuilder.EditDefaultProperty(VirtualTextureThumbnailPropertyHandle)->CustomWidget()
	.ValueContent()
	[
		SNew(SBox)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew( SBox )
			.WidthOverride(64)
			.HeightOverride(64)
			[
				AssetThumbnail->MakeThumbnailWidget(FAssetThumbnailConfig())
			]
		]
	];

	// Call to initialize AssetThumbnail
	RefreshThumbnail();
	DetailBuilder.GetProperty(FName(TEXT("VirtualTexture")))->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FVirtualHeightfieldMeshComponentDetailsCustomization::RefreshThumbnail));

	// Apply custom widget for CopyBounds
	TSharedRef<IPropertyHandle> CopyBoundsPropertyHandle = DetailBuilder.GetProperty(TEXT("bCopyBoundsButton"));
	DetailBuilder.EditDefaultProperty(CopyBoundsPropertyHandle)->CustomWidget()
	.NameContent()
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(LOCTEXT("Button_CopyBounds", "Copy Bounds"))
		.ToolTipText(LOCTEXT("Button_CopyBounds_Tooltip", "Copy the full bounds from the Virtual Texture volume."))
	]
	.ValueContent()
	.MaxDesiredWidth(125.f)
	[
		SNew(SButton)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		.ContentPadding(2)
		.Text(LOCTEXT("Button_CopyBounds", "Copy Bounds"))
		.OnClicked(this, &FVirtualHeightfieldMeshComponentDetailsCustomization::SetBounds)
	];

	// Apply custom widget for BuildMinMaxTexture
	TSharedRef<IPropertyHandle> BuildMinMaxTexturePropertyHandle = DetailBuilder.GetProperty(TEXT("bBuildMinMaxTextureButton"));
	DetailBuilder.EditDefaultProperty(BuildMinMaxTexturePropertyHandle)->CustomWidget()
	.NameContent()
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(LOCTEXT("Button_BuildMinMaxTexture", "Build MinMax Texture"))
		.ToolTipText(LOCTEXT("Button_BuildMinMaxTexture_Tooltip", "Build the MinMax height texture"))
	]
	.ValueContent()
	.MaxDesiredWidth(125.f)
	[
		SNew(SButton)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		.ContentPadding(2)
		.Text(LOCTEXT("Button_Build", "Build"))
		.OnClicked(this, &FVirtualHeightfieldMeshComponentDetailsCustomization::BuildMinMaxTexture)
		.IsEnabled(this, &FVirtualHeightfieldMeshComponentDetailsCustomization::IsMinMaxTextureEnabled)
	];
}

void FVirtualHeightfieldMeshComponentDetailsCustomization::RefreshThumbnail()
{
	AssetThumbnail->SetAsset(VirtualHeightfieldMeshComponent->GetVirtualTexture());
}

FReply FVirtualHeightfieldMeshComponentDetailsCustomization::SetBounds()
{
	ARuntimeVirtualTextureVolume* VirtualTextureVolume = VirtualHeightfieldMeshComponent->GetVirtualTextureVolume();
	if (VirtualTextureVolume != nullptr)
	{
		const FScopedTransaction Transaction(LOCTEXT("Transaction_SetBounds", "Set VirtualHeightfieldMeshComponent Bounds"));
		
		AActor* Owner = VirtualHeightfieldMeshComponent->GetOwner();
		Owner->Modify();
		Owner->SetActorTransform(VirtualTextureVolume->GetTransform());
		Owner->PostEditMove(true);
	
		// Force update of editor view widget.
		GEditor->NoteSelectionChange(false);

		return FReply::Handled();
	}
	return FReply::Unhandled();
}

bool FVirtualHeightfieldMeshComponentDetailsCustomization::IsMinMaxTextureEnabled() const
{
	return VirtualHeightfieldMeshComponent->IsMinMaxTextureEnabled();
}

FReply FVirtualHeightfieldMeshComponentDetailsCustomization::BuildMinMaxTexture()
{
	// Create a new asset if none is already bound
	UHeightfieldMinMaxTexture* CreatedTexture = nullptr;
	if (VirtualHeightfieldMeshComponent->GetMinMaxTexture() == nullptr)
	{
		FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");

		const FString DefaultPath = FPackageName::GetLongPackagePath(VirtualHeightfieldMeshComponent->GetVirtualTexture()->GetPathName());
		const FString DefaultName = FPackageName::GetShortName(VirtualHeightfieldMeshComponent->GetVirtualTexture()->GetName() + TEXT("_MinMax"));

		UFactory* Factory = NewObject<UHeightfieldMinMaxTextureFactory>();
		UObject* Object = AssetToolsModule.Get().CreateAssetWithDialog(DefaultName, DefaultPath, UHeightfieldMinMaxTexture::StaticClass(), Factory);
		CreatedTexture = Cast<UHeightfieldMinMaxTexture>(Object);
	}

	// Build the texture contents
	bool bOK = false;
	if (VirtualHeightfieldMeshComponent->GetMinMaxTexture() != nullptr || CreatedTexture != nullptr)
	{
		const FScopedTransaction Transaction(LOCTEXT("Transaction_BuildMinMaxTexture", "Build MinMax Texture"));

		if (CreatedTexture != nullptr)
		{
			VirtualHeightfieldMeshComponent->Modify();
			VirtualHeightfieldMeshComponent->SetMinMaxTexture(CreatedTexture);
		}

		if (VirtualHeightfieldMesh::BuildMinMaxHeightTexture(VirtualHeightfieldMeshComponent))
		{
			bOK = true;
		}
	}

	return bOK ? FReply::Handled() : FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE
