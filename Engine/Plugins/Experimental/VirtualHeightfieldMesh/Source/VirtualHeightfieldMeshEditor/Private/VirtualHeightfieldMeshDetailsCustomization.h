// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

class FReply;

/** UI customization for UVirtualHeightfieldMeshComponent */
class FVirtualHeightfieldMeshComponentDetailsCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

protected:
	FVirtualHeightfieldMeshComponentDetailsCustomization();

	/** Callback for updating virtual texture thumbnail. */
	void RefreshThumbnail();

	/** Callback for Set Bounds button */
	FReply SetBounds();

	/** Returns true if MinMax texture build button is enabled */
	bool IsMinMaxTextureEnabled() const;
	/** Callback for Build MinMax Texture button */
	FReply BuildMinMaxTexture();

	//~ Begin IDetailCustomization Interface.
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	//~ End IDetailCustomization Interface.

private:
	class UVirtualHeightfieldMeshComponent* VirtualHeightfieldMeshComponent;
	TSharedRef<class FAssetThumbnail> AssetThumbnail;
};
