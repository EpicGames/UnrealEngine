// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualHeightfieldMeshEditorModule.h"

#include "HeightfieldMinMaxTexture.h"
#include "HeightfieldMinMaxTextureAssetTypeActions.h"
#include "HeightfieldMinMaxTextureBuild.h"
#include "HeightfieldMinMaxTextureThumbnailRenderer.h"
#include "PropertyEditorModule.h"
#include "VirtualHeightfieldMeshDetailsCustomization.h"

/** Concrete implementation of the IVirtualHeightfieldMeshEditorModule interface. */
class FVirtualHeightfieldMeshEditorModule : public IVirtualHeightfieldMeshEditorModule
{
public:
	//~ Begin IModuleInterface Interface.
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface Interface.

	//~ Begin IVirtualHeightfieldMeshEditorModule Interface.
	virtual bool HasMinMaxHeightTexture(UVirtualHeightfieldMeshComponent* InComponent) const override;
	virtual bool BuildMinMaxHeightTexture(UVirtualHeightfieldMeshComponent* InComponent) const override;
	//~ End IVirtualHeightfieldMeshEditorModule Interface.
};

IMPLEMENT_MODULE(FVirtualHeightfieldMeshEditorModule, VirtualHeightfieldMeshEditor);

void FVirtualHeightfieldMeshEditorModule::StartupModule()
{
	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	AssetTools.RegisterAssetTypeActions(MakeShareable(new FAssetTypeActions_HeightfieldMinMaxTexture));

	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomClassLayout("VirtualHeightfieldMeshComponent", FOnGetDetailCustomizationInstance::CreateStatic(&FVirtualHeightfieldMeshComponentDetailsCustomization::MakeInstance));

	UThumbnailManager::Get().RegisterCustomRenderer(UHeightfieldMinMaxTexture::StaticClass(), UHeightfieldMinMaxTextureThumbnailRenderer::StaticClass());
}

void FVirtualHeightfieldMeshEditorModule::ShutdownModule()
{
}

bool FVirtualHeightfieldMeshEditorModule::HasMinMaxHeightTexture(UVirtualHeightfieldMeshComponent* InComponent) const
{
	return VirtualHeightfieldMesh::HasMinMaxHeightTexture(InComponent);
}

bool FVirtualHeightfieldMeshEditorModule::BuildMinMaxHeightTexture(UVirtualHeightfieldMeshComponent* InComponent) const
{
	return VirtualHeightfieldMesh::BuildMinMaxHeightTexture(InComponent);
}
