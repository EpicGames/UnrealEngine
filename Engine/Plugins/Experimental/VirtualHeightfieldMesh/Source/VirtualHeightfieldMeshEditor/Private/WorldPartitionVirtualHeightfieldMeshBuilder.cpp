// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartitionVirtualHeightfieldMeshBuilder.h"

#include "HeightfieldMinMaxTexture.h"
#include "HeightfieldMinMaxTextureBuild.h"
#include "PackageSourceControlHelper.h"
#include "UObject/SavePackage.h"
#include "UObject/Package.h"
#include "UObject/UObjectIterator.h"
#include "VirtualHeightfieldMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionVirtualHeightfieldMeshBuilder)

DEFINE_LOG_CATEGORY_STATIC(LogWorldPartitionVirtualHeightfieldMeshBuilder, All, All);

UWorldPartitionVirtualHeightfieldMeshBuilder::UWorldPartitionVirtualHeightfieldMeshBuilder(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UWorldPartitionVirtualHeightfieldMeshBuilder::RunInternal(UWorld* World, const FCellInfo& InCellInfo, FPackageSourceControlHelper& PackageHelper)
{
	// Find UVirtualHeightfieldMeshComponent and update the associated UHeightfieldMinMaxTexture assets.
	// todo[vhm]: Convert builder to sequentially load and build sections of the world.
	TSet<UObject*> ModifiedObjects;
	for (TObjectIterator<UVirtualHeightfieldMeshComponent> It(RF_ClassDefaultObject, false, EInternalObjectFlags::Garbage); It; ++It)
	{
		if (It->GetWorld() == World)
		{
			if (VirtualHeightfieldMesh::HasMinMaxHeightTexture(*It) && VirtualHeightfieldMesh::BuildMinMaxHeightTexture(*It))
			{
				ModifiedObjects.Add(It->GetMinMaxTexture());
			}
		}
	}

	// Checkout and save any modified packages.
	for (UObject* ModifiedObject : ModifiedObjects)
	{
		UObject* Outer = ModifiedObject->GetOuter();
		UPackage* Package = Cast<UPackage>(Outer);

		if (Package != nullptr && Package->IsDirty())
		{
			if (!PackageHelper.Checkout(Package))
			{
				UE_LOGF(LogWorldPartitionVirtualHeightfieldMeshBuilder, Error, "Error checking out package %ls.", *Package->GetName());
				return false;
			}

			FString PackageFileName = SourceControlHelpers::PackageFilename(Package);
			FSavePackageArgs SaveArgs;
			SaveArgs.TopLevelFlags = RF_Standalone;
			SaveArgs.SaveFlags = SAVE_None;
			if (!UPackage::SavePackage(Package, nullptr, *PackageFileName, SaveArgs))
			{
				UE_LOGF(LogWorldPartitionVirtualHeightfieldMeshBuilder, Error, "Error saving package %ls.", *Package->GetName());
				return false;
			}

			if (!PackageHelper.AddToSourceControl(Package))
			{
				UE_LOGF(LogWorldPartitionVirtualHeightfieldMeshBuilder, Error, "Error adding package %ls to revision control.", *Package->GetName());
				return false;
			}
		}
	}

	return true;
}

