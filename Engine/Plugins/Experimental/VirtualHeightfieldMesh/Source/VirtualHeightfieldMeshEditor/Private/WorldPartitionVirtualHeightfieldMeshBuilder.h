// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldPartition/WorldPartitionBuilder.h"
#include "WorldPartitionVirtualHeightfieldMeshBuilder.generated.h"

UCLASS()
class UWorldPartitionVirtualHeightfieldMeshBuilder : public UWorldPartitionBuilder
{
	GENERATED_UCLASS_BODY()

public:
	// UWorldPartitionBuilder interface begin
	virtual bool RequiresCommandletRendering() const override { return true; }
	virtual ELoadingMode GetLoadingMode() const override { return ELoadingMode::EntireWorld; }
protected:
	virtual bool RunInternal(UWorld* World, const FCellInfo& InCellInfo, FPackageSourceControlHelper& PackageHelper) override;
	// UWorldPartitionBuilder interface end
};