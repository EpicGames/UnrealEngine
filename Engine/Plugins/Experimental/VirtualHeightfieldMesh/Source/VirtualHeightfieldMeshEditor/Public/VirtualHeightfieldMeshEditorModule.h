// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

/** Module interface for VirtualHeightfieldMesh editor  extensions. */
class IVirtualHeightfieldMeshEditorModule : public IModuleInterface
{
public:
	/** Returns true if the component describes a runtime virtual texture that has streaming low mips. */
	virtual bool HasMinMaxHeightTexture(class UVirtualHeightfieldMeshComponent* InComponent) const = 0;
	/** Build the contents of the streaming low mips. */
	virtual bool BuildMinMaxHeightTexture(class UVirtualHeightfieldMeshComponent* InComponent) const = 0;
};
