// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class VirtualHeightfieldMeshEditor : ModuleRules
{
	public VirtualHeightfieldMeshEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(new string[] {
			"Core",
			"CoreUObject",		
			"Engine",
			"RenderCore",
			"Renderer",
			"RHI",
			"SourceControl",
			"Slate",
			"SlateCore",
			"UnrealEd",
			"VirtualHeightfieldMesh",
		});
	}
}
