{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Virtual Heightfield Mesh",
	"Description": "Mesh renderer for virtual texture heightfields",
	"Category": "Rendering",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "VirtualHeightfieldMesh",
			"Type": "Runtime",
			"TargetDenyList": [ "Server" ],
			"LoadingPhase": "PostConfigInit"
		},
		{
			"Name": "VirtualHeightfieldMeshEditor",
			"Type": "Editor"
		}
	]
}
